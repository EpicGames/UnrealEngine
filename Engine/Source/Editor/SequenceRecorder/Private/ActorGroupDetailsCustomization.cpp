// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorGroupDetailsCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "DetailWidgetRow.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SequenceRecorder.h"
#include "Widgets/SBoxPanel.h"
#include "EditorFontGlyphs.h"
#include "SequenceRecorderCommands.h"
#include "Widgets/Input/SButton.h"
#include "Framework/Application/SlateApplication.h"
#include "SSequenceRecorder.h"
#include "SequenceRecorderActorGroup.h"
#include "SequenceRecorderUtils.h"

#define LOCTEXT_NAMESPACE "SequenceRecorder"

void FActorGroupDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	// This is displayed via the existing table UI and can't be edited by hand.
	DetailLayout.HideProperty(GET_MEMBER_NAME_CHECKED(USequenceRecorderActorGroup, RecordedActors));

	// This is set via the profile dropdown
	DetailLayout.HideProperty(GET_MEMBER_NAME_CHECKED(USequenceRecorderActorGroup, GroupName));

	DetailLayout.EditCategory(TEXT("Recording Groups"))
		.AddCustomRow(FText::FromString(TEXT("Group Selector"))).WholeRowContent()
		[
			SNew(SHorizontalBox)
			.IsEnabled_Lambda([]() { return !FSequenceRecorder::Get().IsRecording(); })
			.ToolTipText(LOCTEXT("ProfileName", "Select and edit the current sequence recorder group."))
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
			 	SNew(SComboButton)
			 	.ButtonStyle(FAppStyle::Get(), "ToolBar.Button")
			 	.OnGetMenuContent(this, &FActorGroupDetailsCustomization::FillRecordingProfileOptions)
			 	.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
			 	.ButtonContent()
			 	[
			 		SNew(SVerticalBox)
			 		+SVerticalBox::Slot()
			 		.AutoHeight()
			 		.Padding(0.0f, 0.0f, 2.0f, 3.0f)
			 		[
			 			SAssignNew(SequenceRecorderGroupNameTextBox, SEditableTextBox)
			 			.Text_Lambda([]()
			 			{
			 				TWeakObjectPtr<USequenceRecorderActorGroup> Group = FSequenceRecorder::Get().GetCurrentRecordingGroup();
			 				if (Group.IsValid())
			 				{
			 					return FText::FromName(Group->GroupName);	
			 				}
			 
			 				return FText::FromName(NAME_None);
			 			})
			 			.IsEnabled_Lambda([]()
			 			{
			 				return FSequenceRecorder::Get().GetRecordingGroupNames().Num() > 0;
			 			})
			 			.OnTextChanged_Lambda([this](const FText& InText)
			 			{
			 				FName ProfileAsName = *InText.ToString();
			 
			 				// Check to see that no other profile is using this name.
							const FSequenceRecorder& SeqRec = FSequenceRecorder::Get();
			 				if( SeqRec.GetCurrentRecordingGroup().IsValid() &&
								SeqRec.GetCurrentRecordingGroup()->GroupName != ProfileAsName &&
			 					SeqRec.GetRecordingGroupNames().Contains(ProfileAsName))
			 				{
			 					SequenceRecorderGroupNameTextBox->SetError(FText::Format(LOCTEXT("GroupNameAlreadyExists", "Group '{0}' already exists"), InText));
			 				}
			 				else
			 				{
			 					SequenceRecorderGroupNameTextBox->SetError(FText::GetEmpty());
			 				}
			 			})
			 			.OnTextCommitted(FOnTextCommitted::CreateSP(this, &FActorGroupDetailsCustomization::HandleRecordingGroupNameCommitted))
			 		]
			 	]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
			 	CreateRecordingGroupButton(FEditorFontGlyphs::File, FSequenceRecorderCommands::Get().AddRecordingGroup)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
			 	CreateRecordingGroupButton(FEditorFontGlyphs::Trash, FSequenceRecorderCommands::Get().RemoveRecordingGroup)
			]
		];
}

TSharedRef<SWidget> FActorGroupDetailsCustomization::CreateRecordingGroupButton(const FText& InGlyph, TSharedPtr<FUICommandInfo> InCommand)
{
	check(InCommand.IsValid());

	TWeakPtr<FUICommandInfo> LocalCommandPtr = InCommand;
	return SNew(SButton)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.ButtonStyle(FAppStyle::Get(), "FlatButton")
		.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
		.ToolTipText(InCommand->GetDescription())
		.IsEnabled_Lambda([this, LocalCommandPtr]()
		{
			return SequenceRecorder.Pin()->GetCommandList()->CanExecuteAction(LocalCommandPtr.Pin().ToSharedRef());
		})
		.OnClicked(FOnClicked::CreateLambda([this, LocalCommandPtr]()
		{
			return SequenceRecorder.Pin()->GetCommandList()->ExecuteAction(LocalCommandPtr.Pin().ToSharedRef()) ? FReply::Handled() : FReply::Unhandled();
		}))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(InGlyph)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4, 0, 0, 0)
			[
				SNew(STextBlock)
				.Text(InCommand->GetLabel())
			]
		];
}

TSharedRef<SWidget> FActorGroupDetailsCustomization::FillRecordingProfileOptions()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, SequenceRecorder.Pin()->GetCommandList());

	TSharedPtr<SSequenceRecorder> LocalSequenceRecorder = SequenceRecorder.Pin();
	const FSequenceRecorderCommands& Commands = FSequenceRecorderCommands::Get();
	MenuBuilder.BeginSection("CurrentActorGroup", LOCTEXT("CurrentRecordingActorGroupMenu", "Current Actor Group"));
	{
		MenuBuilder.AddMenuEntry(Commands.DuplicateRecordingGroup);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("RecordingActorGroup", LOCTEXT("RecordingActorGroupMenu", "Recording Actor Group"));
	{
		TArray<FName> GroupNames;
		GroupNames.Add(NAME_None);
		GroupNames.Append(FSequenceRecorder::Get().GetRecordingGroupNames());

		// Prevent multiple "None" entries showing up if the user forgot to name a profile.
		for (int32 GroupIdx = GroupNames.Num() - 1; GroupIdx > 0; --GroupIdx)
		{
			if (GroupNames[GroupIdx] == NAME_None)
			{
				GroupNames.RemoveAtSwap(GroupIdx);
			}
		}

		for (FName GroupName : GroupNames)
		{
			FUIAction Action;

			Action.ExecuteAction = FExecuteAction::CreateLambda([GroupName, LocalSequenceRecorder]()
			{
				//Ensure focus is removed because the menu has already closed and the cached value (the one the user has typed) is going to apply to the new profile
				FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::SetDirectly);
				LocalSequenceRecorder->HandleLoadRecordingActorGroup(GroupName);
			});

			Action.GetActionCheckState = FGetActionCheckState::CreateLambda([GroupName]()
			{
				TWeakObjectPtr<USequenceRecorderActorGroup> ActiveGroup = FSequenceRecorder::Get().GetCurrentRecordingGroup();
				return (ActiveGroup.IsValid() && ActiveGroup->GroupName == GroupName) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			});

			TSharedRef<SWidget> RecordingGroupButton =
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(GroupName.ToString()))
				];
			MenuBuilder.AddMenuEntry(Action, RecordingGroupButton, NAME_None, TAttribute<FText>(), EUserInterfaceActionType::Check);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FActorGroupDetailsCustomization::HandleRecordingGroupNameCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	SequenceRecorderGroupNameTextBox->SetError(FText::GetEmpty());

	if (InCommitType != ETextCommit::OnCleared)
	{
		// This is a group re-name operation.
		TWeakObjectPtr<USequenceRecorderActorGroup> CurrentGroup = FSequenceRecorder::Get().GetCurrentRecordingGroup();
		if (CurrentGroup.IsValid())
		{
			const TArray<FName> ExistingGroupNames = FSequenceRecorder::Get().GetRecordingGroupNames();
			const FString NewNameAsString = InText.ToString();
			FName NewName = FName(*NewNameAsString);

			// If they're trying to rename to the same name, just early out. If we try to go through with the rename,
			// it'll see that the name is already in the list and postfix it with "_A" when renaming "Foo" to "Foo".
			if (CurrentGroup->GroupName == NewName)
			{
				return;
			}

			// If they're trying to rename it to something already on our list (another entry) then we forcibly change
			// their name to something unique.
			if (ExistingGroupNames.Contains(NewName))
			{
				NewName = FName(*SequenceRecorderUtils::MakeNewGroupName(*CurrentGroup.Get()->SequenceRecordingBasePath.Path, InText.ToString(), ExistingGroupNames));
			}

			// Re-assign the name of the recording group and update the sequence.
			CurrentGroup->GroupName = NewName;
			CurrentGroup->SequenceName = NewNameAsString;
			CurrentGroup->TargetLevelSequence = nullptr;

			for (UActorRecording* ActorRecording : CurrentGroup->RecordedActors)
			{
				if (ActorRecording != nullptr)
				{
					ActorRecording->TakeNumber = 1;
				}
			}

			// cbb: Force load to update sequence names, etc
			SequenceRecorder.Pin()->HandleLoadRecordingActorGroup(NewName);
		}
	}
}

#undef LOCTEXT_NAMESPACE
