// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Types/SlateEnums.h"

class IDetailLayoutBuilder;
class FUICommandInfo;
class SWidget;
class SEditableTextBox;
class SSequenceRecorder;

class FActorGroupDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<SSequenceRecorder> InSequenceRecorder)
	{
		return MakeShareable(new FActorGroupDetailsCustomization(InSequenceRecorder));
	}

	FActorGroupDetailsCustomization(TWeakPtr<SSequenceRecorder> InSequenceRecorder)
	{
		SequenceRecorder = InSequenceRecorder;
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	TSharedRef<SWidget> CreateRecordingGroupButton(const FText& InGlyph, TSharedPtr<FUICommandInfo> InCommand);
	TSharedRef<SWidget> FillRecordingProfileOptions();
	void HandleRecordingGroupNameCommitted(const FText& InText, ETextCommit::Type InCommitType);
	TSharedPtr<SEditableTextBox> SequenceRecorderGroupNameTextBox;
	TWeakPtr<SSequenceRecorder> SequenceRecorder;
};
