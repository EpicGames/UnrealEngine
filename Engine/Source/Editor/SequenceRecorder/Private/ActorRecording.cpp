// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorRecording.h"
#include "Misc/ScopedSlowTask.h"
#include "GameFramework/Character.h"
#include "Camera/CameraActor.h"
#include "Engine/SimpleConstructionScript.h"
#include "Editor.h"
#include "Animation/SkeletalMeshActor.h"
#include "LevelSequenceBindingReference.h"
#include "SequenceRecorderSettings.h"
#include "SequenceRecorderUtils.h"
#include "Sections/MovieSceneAnimationSectionRecorder.h"
#include "Sections/MovieScene3DTransformSectionRecorderSettings.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "MovieSceneFolder.h"
#include "MovieSceneTimeHelpers.h"
#include "CameraRig_Crane.h"
#include "CameraRig_Rail.h"
#include "SequenceRecorder.h"
#include "Features/IModularFeatures.h"

#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "SequencerUtilities.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/MovieSceneEvaluationState.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorRecording)

static const FName SequencerActorTag(TEXT("SequencerActor"));
static const FName MovieSceneSectionRecorderFactoryName("MovieSceneSectionRecorderFactory");

UActorRecording::UActorRecording(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ActorSettings(this)
{
	TakeNumber = 1;
	bActive = true;
	bCreateLevelSequence = false;
	bWasSpawnedPostRecord = false;
	Guid.Invalidate();
	bNewComponentAddedWhileRecording = false;

	if(!HasAnyFlags(RF_ClassDefaultObject))
	{
		const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();
		AnimationSettings = Settings->DefaultAnimationSettings;
	}
}

bool UActorRecording::IsRelevantForRecording(AActor* Actor)
{
	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

	// don't record actors that sequencer has spawned itself!
	if(!Settings->bRecordSequencerSpawnedActors && Actor->ActorHasTag(SequencerActorTag))
	{
		return false;
	}

	TInlineComponentArray<UActorComponent*> ActorComponents(Actor);
	for(UActorComponent* ActorComponent : ActorComponents)
	{
		for (const FPropertiesToRecordForClass& PropertiesToRecordForClass : Settings->ClassesAndPropertiesToRecord)
		{
			if (PropertiesToRecordForClass.Class != nullptr && ActorComponent->IsA(PropertiesToRecordForClass.Class))
			{
				return true;
			}
		}
	}

	return false;
}

bool UActorRecording::StartRecording(ULevelSequence* CurrentSequence, float CurrentSequenceTime, const FString& BaseAssetPath, const FString& SessionName)
{
	if (!bActive)
	{
		return false;
	}

	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();
	
	AActor* Actor = GetActorToRecord();

	if (Actor && bCreateLevelSequence)
	{
		FString AssetPath = BaseAssetPath / GetTargetName(Actor);

		TakeNumber = SequenceRecorderUtils::GetNewTakeNumber(AssetPath, GetTargetName(Actor), SessionName, TakeNumber);
		FString TakeName = SequenceRecorderUtils::MakeTakeName(GetTargetName(Actor), SessionName, TakeNumber);

		if (TargetLevelSequence != nullptr)
		{
			CurrentSequence = TargetLevelSequence;

			if (ShouldDuplicateLevelSequence())
			{
				IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();

				CurrentSequence = CastChecked<ULevelSequence>(SequenceRecorderUtils::DuplicateAsset(AssetPath, TakeName, TargetLevelSequence));
					
				TargetLevelSequence = CurrentSequence;
			}
		}
		else
		{
			CurrentSequence = SequenceRecorderUtils::MakeNewAsset<ULevelSequence>(AssetPath, TakeName);

			if (CurrentSequence)
			{
				CurrentSequence->Initialize();
			}

			if (bCreateLevelSequence)
			{
				TargetLevelSequence = CurrentSequence;
			}
		}

		if (CurrentSequence)
		{			
			FAssetRegistryModule::AssetCreated(CurrentSequence);
		}
	}

	bNewComponentAddedWhileRecording = false;
	DuplicatedDynamicComponents.Reset();

	if(Actor)
	{
		if (TargetAnimation != nullptr)
		{
			IAssetEditorInstance* EditorInstance = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(TargetAnimation, false);
			if (EditorInstance)
			{
				UE_LOGF(LogAnimation, Log, "Closing '%ls' so we don't invalidate the open version when unloading it.", *TargetAnimation->GetName());
				EditorInstance->CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
			}
		}

		if(CurrentSequence != nullptr)
		{
			StartRecordingActorProperties(CurrentSequence, CurrentSequenceTime);
		}
		else
		{
			FString AnimAssetPath;
			FString AnimAssetName;
			if (TargetLevelSequence != nullptr)
			{
				AnimAssetName = TargetLevelSequence->GetName();
				AnimAssetPath = FPaths::GetPath(TargetLevelSequence->GetPathName());
				if (Settings->AnimationSubDirectory.Len() > 0)
				{
					AnimAssetPath /= Settings->AnimationSubDirectory;
				}
			}

			// If removing root animation from the animation asset, push the root animation 
			// to the transform recorder and the transform track, which gets recorded only 
			// when the animation is not in world space.
			if (AnimationSettings.bRemoveRootAnimation)
			{
				AnimationSettings.bRecordInWorldSpace = false;
			}

			TSharedPtr<FMovieSceneAnimationSectionRecorder> AnimationRecorder = MakeShareable(new FMovieSceneAnimationSectionRecorder(AnimationSettings, TargetAnimation, AnimAssetPath, AnimAssetName));
			AnimationRecorder->CreateSection(GetActorToRecord(), nullptr, FGuid(), 0.0f);
			AnimationRecorder->Record(0.0f);
			SectionRecorders.Add(AnimationRecorder);			
		}
	}

	return true;
}

void UActorRecording::GetNonSceneActorComponents(TArray<UActorComponent*>& OutArray)
{
	if (GetActorToRecord() != nullptr)
	{
		TInlineComponentArray<UActorComponent*> ActorComponents(GetActorToRecord());
		OutArray.Reserve(ActorComponents.Num());
		for(UActorComponent* ActorComponent: ActorComponents)
		{
			if (!ActorComponent->IsA<USceneComponent>())
			{
				 OutArray.Add(ActorComponent);
			}
		}
	}

}
void UActorRecording::GetAllComponents(TArray<UActorComponent*>& OutArray, bool bIncludeNonCDO/*=true*/)
{
	GetSceneComponents(OutArray, bIncludeNonCDO);
	GetNonSceneActorComponents(OutArray);
}

void UActorRecording::GetSceneComponents(TArray<UActorComponent*>& OutArray, bool bIncludeNonCDO/*=true*/)
{
	TArray<USceneComponent*> SceneComponents;
	if(GetActorToRecord() != nullptr)
	{
		USceneComponent* RootComponent = GetActorToRecord()->GetRootComponent();
		if(RootComponent)
		{
			// Get the owned components only. This used to be far more aggressive and
			// would include scene component hierarchy for components not owned by this
			// actor. That led to problems where components would be doubly recorded. 
			GetActorToRecord()->GetComponents(SceneComponents);
			OutArray.Add(RootComponent);
			//Add Scene Components to the OutArray
			for (USceneComponent* SceneComponent : SceneComponents)
			{
				OutArray.Emplace(SceneComponent);
			}
		}

		// add owned components that are *not* part of the hierarchy
		TInlineComponentArray<USceneComponent*> OwnedComponents(GetActorToRecord());
		for(USceneComponent* OwnedComponent : OwnedComponents)
		{
			check(OwnedComponent);
			if(OwnedComponent->GetAttachParent() == nullptr && OwnedComponent != RootComponent)
			{
				OutArray.Add(OwnedComponent);
			}
		}

		if(!bIncludeNonCDO)
		{
			AActor* CDO = Cast<AActor>(GetActorToRecord()->GetClass()->GetDefaultObject());

			auto ShouldRemovePredicate = [&](UActorComponent* PossiblyRemovedComponent)
				{
					if (PossiblyRemovedComponent == nullptr)
					{
						return true;
					}

					// try to find a component with this name in the CDO
					for (UActorComponent* SearchComponent : CDO->GetComponents())
					{
						if (SearchComponent->GetClass() == PossiblyRemovedComponent->GetClass() &&
							SearchComponent->GetFName() == PossiblyRemovedComponent->GetFName())
						{
							return false;
						}
					}

					// remove if its not found
					return true;
				};

			OutArray.RemoveAllSwap(ShouldRemovePredicate);
		}
	}
}

void UActorRecording::SyncTrackedComponents(bool bIncludeNonCDO/*=true*/)
{
	TArray<UActorComponent*> NewComponentArray;
	GetAllComponents(NewComponentArray, bIncludeNonCDO);

	// Expire section recorders that are watching components no longer attached to our actor
	TSet<UActorComponent*> ExpiredComponents;
	for (TWeakObjectPtr<UActorComponent>& WeakComponent : TrackedComponents)
	{
		if (UActorComponent* Component = WeakComponent.Get())
		{
			ExpiredComponents.Add(Component);
		}
	}
	for (UActorComponent* Component : NewComponentArray)
	{
		ExpiredComponents.Remove(Component);
	}

	for (TSharedPtr<IMovieSceneSectionRecorder>& SectionRecorder : SectionRecorders)
	{
		if (UActorComponent* Component = Cast<UActorComponent>(SectionRecorder->GetSourceObject()))
		{
			if (ExpiredComponents.Contains(Component))
			{
				SectionRecorder->InvalidateObjectToRecord();
			}
		}
	}

	TrackedComponents.Reset(NewComponentArray.Num());
	for(UActorComponent* ActorComponent : NewComponentArray)
	{
		TrackedComponents.Add(ActorComponent);
	}
}

void UActorRecording::InvalidateObjectToRecord()
{
	ActorToRecord = nullptr;
	for(auto& SectionRecorder : SectionRecorders)
	{
		SectionRecorder->InvalidateObjectToRecord();
	}
}

bool UActorRecording::ValidComponent(UActorComponent* ActorComponent) const
{
	if(ActorComponent != nullptr)
	{
		const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();
		for (const FPropertiesToRecordForClass& PropertiesToRecordForClass : Settings->ClassesAndPropertiesToRecord)
		{			
			if (PropertiesToRecordForClass.Class != nullptr && ActorComponent->IsA(PropertiesToRecordForClass.Class) && !ActorComponent->bIsEditorOnly)
			{
				return true;
			}
		}
	}

	return false;
}

void UActorRecording::FindOrAddFolder(UMovieScene* MovieScene)
{
	check(GetActorToRecord() != nullptr);

	FName FolderName(NAME_None);
	if(GetActorToRecord()->IsA<ACharacter>() || GetActorToRecord()->IsA<ASkeletalMeshActor>())
	{
		FolderName = TEXT("Characters");
	}
	else if(GetActorToRecord()->IsA<ACameraActor>() || GetActorToRecord()->IsA<ACameraRig_Crane>() || GetActorToRecord()->IsA<ACameraRig_Rail>())
	{
		FolderName = TEXT("Cameras");
	}
	else
	{
		FolderName = TEXT("Misc");
	}

	// look for a folder to put us in
	UMovieSceneFolder* FolderToUse = nullptr;
	for(UMovieSceneFolder* Folder : MovieScene->GetRootFolders())
	{
		if(Folder->GetFolderName() == FolderName)
		{
			FolderToUse = Folder;
			break;
		}
	}

	if(FolderToUse == nullptr)
	{
		FolderToUse = NewObject<UMovieSceneFolder>(MovieScene, NAME_None, RF_Transactional);
		FolderToUse->SetFolderName(FolderName);
		MovieScene->AddRootFolder(FolderToUse);
	}

	FolderToUse->AddChildObjectBinding(Guid);
}

ULevelSequence* UActorRecording::GetActiveLevelSequence(ULevelSequence* InLevelSequence)
{
	if (TargetLevelSequence != nullptr)
	{
		return TargetLevelSequence;
	}

	return InLevelSequence;
}

bool UActorRecording::ShouldDuplicateLevelSequence()
{
	// Duplicate the level sequence if the take number we're trying to write to is not the same as our existing take number
	if (TargetLevelSequence != nullptr)
	{
		FString TakeName = TargetLevelSequence->GetName();
		FString TargetActorName;
		FString TargetSessionName;
		uint32 TargetTakeNumber;
		if (SequenceRecorderUtils::ParseTakeName(TakeName, TargetActorName, TargetSessionName, TargetTakeNumber))
		{
			if (TargetTakeNumber != TakeNumber)
			{
				return true;
			}
		}
	}

	return false;
}


FString UActorRecording::GetTargetName(AActor* InActor)
{
	if (!TargetName.IsEmpty())
	{
		return TargetName.ToString();
	}

	if (InActor)
	{
		return InActor->GetActorLabel();
	}

	return FString();
}


FGuid UActorRecording::GetActorInSequence(AActor* InActor, ULevelSequence* CurrentSequence)
{
	FString ActorTargetName = GetTargetName(InActor);

	CurrentSequence = GetActiveLevelSequence(CurrentSequence);

	UMovieScene* MovieScene = CurrentSequence->GetMovieScene();
	
	for (int32 PossessableCount = 0; PossessableCount < MovieScene->GetPossessableCount(); ++PossessableCount)
	{
		const FMovieScenePossessable& Possessable = MovieScene->GetPossessable(PossessableCount);
		if (Possessable.GetName() == ActorTargetName || Possessable.Tags.Contains(*InActor->GetActorLabel()))
		{
			return Possessable.GetGuid();
		}
	}
	return FGuid();
}

void UActorRecording::StartRecordingActorProperties(ULevelSequence* CurrentSequence, float CurrentSequenceTime)
{
	CurrentSequence = GetActiveLevelSequence(CurrentSequence);

	if(CurrentSequence != nullptr)
	{
		// set up our spawnable or possessable for this actor
		UMovieScene* MovieScene = CurrentSequence->GetMovieScene();

		AActor* Actor = GetActorToRecord();

		// Look for an existing guid in the current sequence to record to 
		FString ObjectBindingName = GetTargetName(Actor);
		Guid = GetActorInSequence(Actor, CurrentSequence);


		if (!Guid.IsValid())
		{
			if (bRecordToPossessable)
			{ 
				UWorld* World = Actor->GetWorld();
				Guid = MovieScene->AddPossessable(ObjectBindingName, Actor->GetClass());
				CurrentSequence->BindPossessableObject(Guid, *Actor, Actor->GetWorld());
				MovieScene->FindPossessable(Guid)->FixupPossessedObjectClass(CurrentSequence, World);
			}
			else
			{
				FString TemplateName = MovieSceneHelpers::MakeUniqueBindingName(MovieScene, Actor->GetName());

				AActor* ObjectTemplate = CastChecked<AActor>(CurrentSequence->MakeSpawnableTemplateFromInstance(*Actor, *TemplateName));

				if (ObjectTemplate)
				{
					for (UActorComponent* Component : ObjectTemplate->GetComponents())
					{
						if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Component))
						{
							SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
							SkeletalMeshComponent->bEnableUpdateRateOptimizations = false;
							SkeletalMeshComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
							SkeletalMeshComponent->SetForcedLOD(1);
						}
					}

					// Disable possession of pawns otherwise the recorded character will auto possess the player
					if (ObjectTemplate->IsA(APawn::StaticClass()))
					{
						APawn* Pawn = CastChecked<APawn>(ObjectTemplate);
						Pawn->AutoPossessPlayer = EAutoReceiveInput::Disabled;
					}

					UE::Sequencer::FCreateBindingParams CreateBindingParams;
					CreateBindingParams.bAllowCustomBinding = true;
					CreateBindingParams.bSpawnable = true;
					CreateBindingParams.BindingNameOverride = ObjectBindingName;
					Guid = FSequencerUtilities::CreateOrReplaceBinding(nullptr, CurrentSequence, ObjectTemplate, CreateBindingParams);
				}
			}
		}

		// now add tracks to record
		if(Guid.IsValid())
		{
			// Tag the possessable/spawnable with the original actor label so we can find it later
			if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(Guid))
			{
				if (!Possessable->Tags.Contains(*Actor->GetActorLabel()))
				{
					Possessable->Tags.AddUnique(FName(*Actor->GetActorLabel()));
				}
				Possessable->SetName(ObjectBindingName);
			}

			// add our folder
			FindOrAddFolder(MovieScene);

			// force set recording to record translations as we need this with no animation
			UMovieScene3DTransformSectionRecorderSettings* TransformSettings = ActorSettings.GetSettingsObject<UMovieScene3DTransformSectionRecorderSettings>();
			check(TransformSettings);
			if (!TransformSettings->bRecordTransforms)
			{
				UE_LOGF(LogAnimation, Warning, "Transform recording is not enabled for '%ls'. Resulting animation may not match gameplay.", *Actor->GetActorLabel());
			}

			// grab components so we can track attachments
			// don't include non-CDO here as they wont be part of our initial BP (duplicated above)
			// we will catch these 'extra' components on the first tick
			const bool bIncludeNonCDO = false;
			SyncTrackedComponents(bIncludeNonCDO);

			TInlineComponentArray<USceneComponent*> SceneComponents(GetActorToRecord());

			// check if components need recording
			TInlineComponentArray<UActorComponent*> ValidActorComponents;
			for(TWeakObjectPtr<UActorComponent>& ActorComponent : TrackedComponents)
			{
				if(ValidComponent(ActorComponent.Get()))
				{
					ValidActorComponents.Add(ActorComponent.Get());

					// add all parent components too
					USceneComponent *SceneComponent = Cast<USceneComponent>(ActorComponent);
					if (SceneComponent)
					{
						TArray<USceneComponent*> ParentComponents;
						SceneComponent->GetParentComponents(ParentComponents);
						for (USceneComponent* ParentComponent : ParentComponents)
						{
							ValidActorComponents.AddUnique(ParentComponent);
						}
					}
				}
			}

			ProcessNewComponentArray(ValidActorComponents);

			TSharedPtr<FMovieSceneAnimationSectionRecorder> FirstAnimRecorder = nullptr;
			for(UActorComponent* ActorComponent : ValidActorComponents)
			{
				TSharedPtr<FMovieSceneAnimationSectionRecorder> AnimRecorder = StartRecordingComponentProperties(ActorComponent->GetFName(), ActorComponent, GetActorToRecord(), CurrentSequence, CurrentSequenceTime, AnimationSettings, TargetAnimation);
				if(!FirstAnimRecorder.IsValid() && AnimRecorder.IsValid() && GetActorToRecord()->IsA<ACharacter>())
				{
					FirstAnimRecorder = AnimRecorder;
				}
			}

			// we need to create a transform track even if we arent recording transforms
			if (FSequenceRecorder::Get().GetTransformRecorderFactory().CanRecordObject(GetActorToRecord()))
			{
				TSharedPtr<IMovieSceneSectionRecorder> Recorder = FSequenceRecorder::Get().GetTransformRecorderFactory().CreateSectionRecorder(TransformSettings->bRecordTransforms, FirstAnimRecorder);
				if(Recorder.IsValid())
				{ 
					Recorder->CreateSection(GetActorToRecord(), MovieScene, Guid, CurrentSequenceTime);
					Recorder->Record(CurrentSequenceTime);
					SectionRecorders.Add(Recorder);
				}
			}

			TArray<IMovieSceneSectionRecorderFactory*> ModularFeatures = IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneSectionRecorderFactory>(MovieSceneSectionRecorderFactoryName);
			for (IMovieSceneSectionRecorderFactory* Factory : ModularFeatures)
			{
				if (Factory->CanRecordObject(GetActorToRecord()))
				{
					TSharedPtr<IMovieSceneSectionRecorder> Recorder = Factory->CreateSectionRecorder(ActorSettings);
					if (Recorder.IsValid())
					{
						Recorder->CreateSection(GetActorToRecord(), MovieScene, Guid, CurrentSequenceTime);
						Recorder->Record(CurrentSequenceTime);
						SectionRecorders.Add(Recorder);
					}
				}
			}	
		}
	}
}

TSharedPtr<FMovieSceneAnimationSectionRecorder> UActorRecording::StartRecordingComponentProperties(const FName& BindingName, UActorComponent* ActorComponent, UObject* BindingContext, ULevelSequence* CurrentSequence, float CurrentSequenceTime, const FAnimationRecordingSettings& InAnimationSettings, UAnimSequence* InTargetSequence)
{
	CurrentSequence = GetActiveLevelSequence(CurrentSequence);

	// first create a possessable for this component to be controlled by
	UMovieScene* OwnerMovieScene = CurrentSequence->GetMovieScene();

	FGuid PossessableGuid;
	for (int32 PossessableCount = 0; PossessableCount < OwnerMovieScene->GetPossessableCount(); ++PossessableCount)
	{
		const FMovieScenePossessable& Possessable = OwnerMovieScene->GetPossessable(PossessableCount);
		if (Possessable.GetParent() == Guid && Possessable.GetName() == BindingName.ToString() && Possessable.GetPossessedObjectClass() == ActorComponent->GetClass())
		{
			PossessableGuid = Possessable.GetGuid();
			break;
		}
	}
	
	if (!PossessableGuid.IsValid())
	{
		PossessableGuid = OwnerMovieScene->AddPossessable(BindingName.ToString(), ActorComponent->GetClass());
	}

	// Set up parent/child guids for possessables within spawnables
	FMovieScenePossessable* ChildPossessable = OwnerMovieScene->FindPossessable(PossessableGuid);
	if (ensure(ChildPossessable))
	{
		ChildPossessable->SetParent(Guid, OwnerMovieScene);
	}

	CurrentSequence->BindPossessableObject(PossessableGuid, *ActorComponent, BindingContext);

	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();
	
	// First try built-in animation recorder...
	TSharedPtr<FMovieSceneAnimationSectionRecorder> AnimationRecorder = nullptr;
	if (FSequenceRecorder::Get().GetAnimationRecorderFactory().CanRecordObject(ActorComponent))
	{
		FString AnimAssetPath;
		FString AnimAssetName;
		if (TargetLevelSequence != nullptr)
		{
			AnimAssetName = TargetLevelSequence->GetName();
			AnimAssetPath = FPaths::GetPath(TargetLevelSequence->GetPathName());
			if (Settings->AnimationSubDirectory.Len() > 0)
			{
				AnimAssetPath /= Settings->AnimationSubDirectory;
			}
		}

		AnimationRecorder = FSequenceRecorder::Get().GetAnimationRecorderFactory().CreateSectionRecorder(InTargetSequence, InAnimationSettings, AnimAssetPath, AnimAssetName);
		AnimationRecorder->CreateSection(ActorComponent, OwnerMovieScene, PossessableGuid, CurrentSequenceTime);
		AnimationRecorder->Record(CurrentSequenceTime);
		SectionRecorders.Add(AnimationRecorder);
	}

	// ...and transform...
	if (FSequenceRecorder::Get().GetTransformRecorderFactory().CanRecordObject(ActorComponent))
	{
		TSharedPtr<IMovieSceneSectionRecorder> Recorder = FSequenceRecorder::Get().GetTransformRecorderFactory().CreateSectionRecorder(true, nullptr);
		if (Recorder.IsValid())
		{
			Recorder->CreateSection(ActorComponent, OwnerMovieScene, PossessableGuid, CurrentSequenceTime);
			Recorder->Record(CurrentSequenceTime);
			SectionRecorders.Add(Recorder);
		}
	}

	// ...now any external recorders
	TArray<IMovieSceneSectionRecorderFactory*> ModularFeatures = IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneSectionRecorderFactory>(MovieSceneSectionRecorderFactoryName);
	for (IMovieSceneSectionRecorderFactory* Factory : ModularFeatures)
	{
		if (Factory->CanRecordObject(ActorComponent))
		{
			TSharedPtr<IMovieSceneSectionRecorder> Recorder = Factory->CreateSectionRecorder(ActorSettings);
			if (Recorder.IsValid())
			{
				Recorder->CreateSection(ActorComponent, OwnerMovieScene, PossessableGuid, CurrentSequenceTime);
				Recorder->Record(CurrentSequenceTime);
				SectionRecorders.Add(Recorder);
			}
		}
	}

	return AnimationRecorder;
}

void UActorRecording::Tick(ULevelSequence* CurrentSequence, float CurrentSequenceTime)
{
	if (IsRecording())
	{
		CurrentSequence = GetActiveLevelSequence(CurrentSequence);

		if(CurrentSequence)
		{
			// check our components to see if they have changed
			static TArray<UActorComponent*> ActorCompnents;
			GetAllComponents(ActorCompnents);

			if(TrackedComponents.Num() != ActorCompnents.Num())
			{
				StartRecordingNewComponents(CurrentSequence, CurrentSequenceTime);
			}
		}

		for(auto& SectionRecorder : SectionRecorders)
		{
			SectionRecorder->Record(CurrentSequenceTime);
		}
	}
}

bool UActorRecording::StopRecording(ULevelSequence* OriginalSequence, float CurrentSequenceTime)
{
	if (!bActive)
	{
		return false;
	}

	ULevelSequence* CurrentSequence = GetActiveLevelSequence(OriginalSequence);

	FString ActorName;
	if(CurrentSequence)
	{
		UMovieScene* MovieScene = CurrentSequence->GetMovieScene();
		check(MovieScene);

		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(Guid);
		if(Possessable)
		{
			ActorName = Possessable->GetName();
		}
	}

	FScopedSlowTask SlowTask((float)SectionRecorders.Num() + 1.0f, FText::Format(NSLOCTEXT("SequenceRecorder", "ProcessingActor", "Processing Actor {0}"), FText::FromString(ActorName)));

	// stop property recorders
	for(auto& SectionRecorder : SectionRecorders)
	{
		SlowTask.EnterProgressFrame();

		SectionRecorder->FinalizeSection(CurrentSequenceTime);
	}

	SlowTask.EnterProgressFrame();

	SectionRecorders.Empty();

	if (TargetLevelSequence != nullptr)
	{
		// set movie scene playback range to encompass all sections
		SequenceRecorderUtils::ExtendSequencePlaybackRange(TargetLevelSequence);

		const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();
		if(Settings->bAutoSaveAsset || GEditor == nullptr)
		{
			SequenceRecorderUtils::SaveAsset(TargetLevelSequence);
		}
	}

	// Add this sequence as a subtrack
	if (CurrentSequence && OriginalSequence && OriginalSequence != CurrentSequence)
	{
		UMovieScene* MovieScene = OriginalSequence->GetMovieScene();
		UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(MovieScene->FindTrack(UMovieSceneSubTrack::StaticClass()));
		if (!SubTrack)
		{
			SubTrack = Cast<UMovieSceneSubTrack>(MovieScene->AddTrack(UMovieSceneSubTrack::StaticClass()));
		}

		// Remove the current take if it exists
		FString CurrentActorName;
		FString CurrentSessionName;
		uint32 CurrentTakeNumber;
		TOptional<int32> RowIndex;
		TOptional<int32> NewTakeNumber;
		if (SequenceRecorderUtils::ParseTakeName(CurrentSequence->GetName(), CurrentActorName, CurrentSessionName, CurrentTakeNumber))
		{
			NewTakeNumber = CurrentTakeNumber + 1;

			for (UMovieSceneSection* Section : SubTrack->GetAllSections())
			{
				UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
				UMovieSceneSequence* SubSequence = SubSection->GetSequence();
				if (SubSequence)
				{
					FString SubActorName;
					FString SubSessionName;
					uint32 SubTakeNumber;
					if (SequenceRecorderUtils::ParseTakeName(SubSequence->GetName(), SubActorName, SubSessionName, SubTakeNumber))
					{
						if (SubActorName == CurrentActorName && SubSessionName == CurrentSessionName)
						{
							RowIndex = Section->GetRowIndex();
							SubTrack->RemoveSection(*Section);
							break;
						}
					}
				}
			}
		}

		// Add new take sub section
		if (!RowIndex.IsSet())
		{
			RowIndex = SubTrack->GetMaxRowIndex() + 1;
		}

		FFrameNumber RecordStartTime = OriginalSequence->GetMovieScene()->GetPlaybackRange().GetLowerBoundValue();
		UMovieSceneSubSection* SubSection = SubTrack->AddSequence(CurrentSequence, RecordStartTime,  UE::MovieScene::DiscreteSize(CurrentSequence->GetMovieScene()->GetPlaybackRange()));

		if (SubSection->GetAutoSizeRange().IsSet())
		{
			SubSection->SetRange(SubSection->GetAutoSizeRange().GetValue());
		}

		SubSection->SetRowIndex(RowIndex.GetValue());

		SubTrack->FixRowIndices();

		// Increment the take number for the next recording
		if (NewTakeNumber.IsSet())
		{
			TakeNumber = NewTakeNumber.GetValue();
		}
	}

	// Swap to editor actor in case the actor was set while in PIE
	if (AActor* Actor = ActorToRecord.Get())
	{
		if (AActor* EditorActor = EditorUtilities::GetEditorWorldCounterpartActor(Actor))
		{
			ActorToRecord = EditorActor;
		}
	}

	return true;
}

bool UActorRecording::IsRecording() const
{
	return GetActorToRecord() && SectionRecorders.Num() > 0;
}

AActor* UActorRecording::GetActorToRecord() const
{
	if (AActor* AssignedActor = ActorToRecord.Get())
	{
		AActor* OutActor = EditorUtilities::GetSimWorldCounterpartActor(AssignedActor);

		if (OutActor != nullptr)
		{
			return OutActor;
		}

		return AssignedActor;
	}

	return nullptr;
}

void UActorRecording::SetActorToRecord(AActor* InActor)
{
	ActorToRecord = InActor; 

	if (InActor != nullptr)
	{
		bRecordToPossessable = false;

		const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();
		for (const FSettingsForActorClass& SettingsForActorClass : Settings->PerActorSettings)
		{
			if (InActor->GetClass()->IsChildOf(SettingsForActorClass.Class))
			{
				bRecordToPossessable = SettingsForActorClass.bRecordToPossessable;
			}
		}
	}
}

void UActorRecording::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UActorRecording, ActorToRecord))
	{
		if (AActor* Actor = GetActorToRecord())
		{
			bRecordToPossessable = false;

			const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();
			for (const FSettingsForActorClass& SettingsForActorClass : Settings->PerActorSettings)
			{
				if (Actor->GetClass()->IsChildOf(SettingsForActorClass.Class))
				{
					bRecordToPossessable = SettingsForActorClass.bRecordToPossessable;
				}
			}
		}
	}
}

static FName FindParentComponentOwnerClassName(UActorComponent* ActorComponent, UBlueprint* Blueprint)
{
	USceneComponent* SceneComponent = Cast<USceneComponent>(ActorComponent);
	if(SceneComponent->GetAttachParent())
	{
		FName AttachName = SceneComponent->GetAttachParent()->GetFName();

		// see if we can find this component in the BP inheritance hierarchy
		while(Blueprint)
		{
			if(Blueprint->SimpleConstructionScript->FindSCSNode(AttachName) != nullptr)
			{
				return Blueprint->GetFName();
			}

			Blueprint = Cast<UBlueprint>(Blueprint->GeneratedClass->GetSuperClass()->ClassGeneratedBy);
		}
	}

	return NAME_None;
}

int32 GetAttachmentDepth(USceneComponent* Component)
{
	int32 Depth = 0;

	USceneComponent* Parent = Component->GetAttachParent();
	while (Parent)
	{
		++Depth;
		Parent = Parent->GetAttachParent();
	}

	return Depth;
}

void UActorRecording::ProcessNewComponentArray(TInlineComponentArray<UActorComponent*>& ProspectiveComponents) const
{
	// Only iterate as far as the current size of the array (it may grow inside the loop)
	int32 LastIndex = ProspectiveComponents.Num();
	for (int32 Index = 0; Index < LastIndex; ++Index)
	{
		USceneComponent* NewComponent = Cast<USceneComponent>(ProspectiveComponents[Index]);
		if (NewComponent)
		{
			USceneComponent* Parent = NewComponent->GetAttachParent();

			while (Parent)
			{
				TWeakObjectPtr<USceneComponent> WeakParent(Parent);
				if (TrackedComponents.Contains(WeakParent) || ProspectiveComponents.Contains(Parent) || Parent->GetOwner() != NewComponent->GetOwner())
				{
					break;
				}
				else
				{
					ProspectiveComponents.Add(Parent);
				}

				Parent = Parent->GetAttachParent();
			}

		}
	}
	// Sort parent first, to ensure that attachments get added properly
	TMap<UActorComponent*, int32> AttachmentDepths;
	for (UActorComponent* ActorComponent : ProspectiveComponents)
	{
		USceneComponent* SceneComponent = Cast<USceneComponent>(ActorComponent);
		if (SceneComponent)
		{
			AttachmentDepths.Add(ActorComponent, GetAttachmentDepth(SceneComponent));
		}
		else
		{
			AttachmentDepths.Add(ActorComponent, 0);
		}
	}

	ProspectiveComponents.Sort(
		[&](UActorComponent& A, UActorComponent& B)
		{
			return *AttachmentDepths.Find(&A) < *AttachmentDepths.Find(&B);
		}
	);
}

void UActorRecording::StartRecordingNewComponents(ULevelSequence* CurrentSequence, float CurrentSequenceTime)
{
	CurrentSequence = GetActiveLevelSequence(CurrentSequence);

	if (GetActorToRecord() != nullptr)
	{
		// find the new component(s)
		TInlineComponentArray<UActorComponent*> NewComponents;
		TArray<UActorComponent*> ActorComponents;
		GetAllComponents(ActorComponents);
		for(UActorComponent* ActorComponent : ActorComponents)
		{
			if(ValidComponent(ActorComponent))
			{
				TWeakObjectPtr<UActorComponent> WeakActorComponent(ActorComponent);
				int32 FoundIndex = TrackedComponents.Find(WeakActorComponent);
				if(FoundIndex == INDEX_NONE)
				{
					// new component!
					NewComponents.Add(ActorComponent);
				}
			}
		}

		ProcessNewComponentArray(NewComponents);

		UMovieScene* MovieScene = CurrentSequence->GetMovieScene();
		check(MovieScene);

		FAnimationRecordingSettings ComponentAnimationSettings = AnimationSettings;
		ComponentAnimationSettings.bRemoveRootAnimation = false;
		ComponentAnimationSettings.bRecordInWorldSpace = false;

		const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();
		if (!bRecordToPossessable)
		{
			TSharedRef<UE::MovieScene::FSharedPlaybackState> TransientPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(GetActorToRecord()->GetWorld(), CurrentSequence);

			AActor* ObjectTemplate = Cast<AActor>(MovieSceneHelpers::GetObjectTemplate(CurrentSequence, Guid, TransientPlaybackState, 0));

			for (UActorComponent* ActorComponent : NewComponents)
			{
				// new component, so we need to add this to our BP if it didn't come from SCS
				FName NewName;
				if (ActorComponent->CreationMethod != EComponentCreationMethod::SimpleConstructionScript)
				{
					// Give this component a unique name within its parent
					NewName = *FString::Printf(TEXT("Dynamic%s"), *ActorComponent->GetFName().GetPlainNameString());
					NewName.SetNumber(1);
					while (FindObjectFast<UObject>(ObjectTemplate, NewName))
					{
						NewName.SetNumber(NewName.GetNumber() + 1);
					}

					USceneComponent* TemplateRoot = ObjectTemplate->GetRootComponent();
					USceneComponent* AttachToComponent = nullptr;

					// look for a similar attach parent in the current structure
					USceneComponent* SceneComponent = Cast<USceneComponent>(ActorComponent);
					if (SceneComponent)
					{
						USceneComponent* AttachParent = SceneComponent->GetAttachParent();
						if (AttachParent != nullptr)
						{
							// First off, check if we're attached to a component that has already been duplicated into this object
							// If so, the name lookup will fail, so we use a direct reference
							if (TWeakObjectPtr<UActorComponent>* DuplicatedComponent = DuplicatedDynamicComponents.Find(AttachParent))
							{
								UActorComponent* LocalActorComponent = DuplicatedComponent->Get();
								AttachToComponent = Cast<USceneComponent>(LocalActorComponent);
							}

							// If we don't have an attachment parent duplicated already, perform a name lookup
							if (!AttachToComponent)
							{
								FName AttachName = SceneComponent->GetAttachParent()->GetFName();

								for (UActorComponent* Component : ObjectTemplate->GetComponents())
								{
									if (USceneComponent* Child = Cast<USceneComponent>(Component))
									{
										CA_SUPPRESS(28182); // Dereferencing NULL pointer. 'Child' contains the same NULL value as 'AttachToComponent' did.
										if (Child->GetFName() == AttachName)
										{
											AttachToComponent = Child;
											break;
										}
									}
								}
							}
						}

						if (!AttachToComponent)
						{
							AttachToComponent = ObjectTemplate->GetRootComponent();
						}

						USceneComponent* NewTemplateComponent = Cast<USceneComponent>(StaticDuplicateObject(SceneComponent, ObjectTemplate, NewName, RF_AllFlags & ~RF_Transient));
						NewTemplateComponent->AttachToComponent(AttachToComponent, FAttachmentTransformRules::KeepRelativeTransform, SceneComponent->GetAttachSocketName());

						ObjectTemplate->AddInstanceComponent(NewTemplateComponent);

						DuplicatedDynamicComponents.Add(ActorComponent, TWeakObjectPtr< USceneComponent >( NewTemplateComponent ) );
					}
				}
				else
				{
					NewName = ActorComponent->GetFName();
				}

				StartRecordingComponentProperties(NewName, ActorComponent, GetActorToRecord(), CurrentSequence, CurrentSequenceTime, ComponentAnimationSettings, nullptr);

				bNewComponentAddedWhileRecording = true;
			}

			SyncTrackedComponents();
		}
		else
		{
			for (UActorComponent* ActorComponent : NewComponents)
			{
				// new component, start recording
				StartRecordingComponentProperties(ActorComponent->GetFName(), ActorComponent, GetActorToRecord(), CurrentSequence, CurrentSequenceTime, ComponentAnimationSettings, nullptr);
			}

			SyncTrackedComponents();
		}
	}
}
