// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorRecordingDetailsCustomization.h"

#include "ActorRecording.h"
#include "Containers/Array.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "ObjectEditorUtils.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

void FActorRecordingDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<class IPropertyHandle> ActorSettingsHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UActorRecording, ActorSettings));
	ActorSettingsHandle->MarkHiddenByCustomization();
	TSharedPtr<class IPropertyHandle> SettingsHandle = ActorSettingsHandle->GetChildHandle("Settings");	// cant use GET_MEMBER_NAME_CHECKED here because of private access
	SettingsHandle->MarkHiddenByCustomization();

	uint32 NumSettingsObjects;
	SettingsHandle->GetNumChildren(NumSettingsObjects);
	for (uint32 SettingsObjectIndex = 0; SettingsObjectIndex < NumSettingsObjects; ++SettingsObjectIndex)
	{
		TSharedPtr<class IPropertyHandle> SettingsObjectHandle = SettingsHandle->GetChildHandle(SettingsObjectIndex);
		UObject* SettingsObject = nullptr;
		if (SettingsObjectHandle->GetValue(SettingsObject) == FPropertyAccess::Success && SettingsObject != nullptr)
		{
			TArray<UObject*> ObjectArray;
			ObjectArray.Add(SettingsObject);
			for (TFieldIterator<FProperty> PropertyIt(SettingsObject->GetClass()); PropertyIt; ++PropertyIt)
			{
				IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(FObjectEditorUtils::GetCategoryFName(*PropertyIt));
				CategoryBuilder.AddExternalObjectProperty(ObjectArray, PropertyIt->GetFName());
			}
		}
	}
}
