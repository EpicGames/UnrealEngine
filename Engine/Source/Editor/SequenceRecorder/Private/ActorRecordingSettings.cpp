// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorRecordingSettings.h"

#include "Features/IModularFeatures.h"
#include "HAL/PlatformCrt.h"
#include "IMovieSceneSectionRecorderFactory.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorRecordingSettings)

FActorRecordingSettings::FActorRecordingSettings()
{
}

FActorRecordingSettings::FActorRecordingSettings(UObject* InOuter)
{
	Outer = InOuter;
}

void FActorRecordingSettings::CreateSettingsObjectsFromFactory()
{
	TArray<IMovieSceneSectionRecorderFactory*> ModularFeatures = IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneSectionRecorderFactory>("MovieSceneSectionRecorderFactory");
	for (IMovieSceneSectionRecorderFactory* Factory : ModularFeatures)
	{
		UObject* SettingsObject = Factory->CreateSettingsObject(Outer.Get() ? Outer.Get() : (UObject*)GetTransientPackage());
		if (SettingsObject)
		{
			Settings.Add(SettingsObject);
		}
	}
}

void FActorRecordingSettings::CreateSettingsObjectsIfNeeded() const
{
	if (Settings.Num() == 0)
	{
		const_cast<FActorRecordingSettings*>(this)->CreateSettingsObjectsFromFactory();
	}
}
