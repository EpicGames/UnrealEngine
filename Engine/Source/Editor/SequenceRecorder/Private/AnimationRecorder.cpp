// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationRecorder.h"
#include "Misc/QualifiedFrameTime.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Animation/AnimSequence.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/MessageDialog.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "Misc/PackageName.h"
#include "Editor.h"

#include "Animation/AnimInstance.h"
#include "Animation/AnimCompress.h"
#include "Animation/AnimCompress_BitwiseCompressOnly.h"
#include "SCreateAnimationDlg.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Animation/AnimationRecordingSettings.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "AnimationRecorderParameters.h"

#define LOCTEXT_NAMESPACE "FAnimationRecorder"

static TAutoConsoleVariable<int32> CVarKeepNotifyAndCurvesOnAnimationRecord(
	TEXT("a.KeepNotifyAndCurvesOnAnimationRecord"),
	1,
	TEXT("If nonzero we keep anim notifies, curves and sync markers when animation recording, if 0 we discard them before recording."),
	ECVF_Default);

/////////////////////////////////////////////////////

FAnimationRecorder::FAnimationRecorder()
	: AnimationObject(nullptr)
	, bRecordLocalToWorld(false)
	, bAutoSaveAsset(false)
	, bRemoveRootTransform(true)
	, bCheckDeltaTimeAtBeginning(true)
	, Interpolation(EAnimInterpolationType::Linear)
	, InterpMode(ERichCurveInterpMode::RCIM_Linear)
	, TangentMode(ERichCurveTangentMode::RCTM_Auto)
	, AnimationSerializer(nullptr)
{
	SetSampleRateAndLength(FAnimationRecordingSettings::DefaultSampleFrameRate, FAnimationRecordingSettings::DefaultMaximumLength);
}

FAnimationRecorder::~FAnimationRecorder()
{
	StopRecord(false);
}

void FAnimationRecorder::SetSampleRateAndLength(FFrameRate SampleFrameRate, float LengthInSeconds)
{
	if (!SampleFrameRate.IsValid())
	{
		// invalid rate passed in, fall back to default
		SampleFrameRate = FAnimationRecordingSettings::DefaultSampleFrameRate;
	}

	if (LengthInSeconds <= 0.f)
	{
		// invalid length passed in, default to unbounded
		LengthInSeconds = FAnimationRecordingSettings::UnboundedMaximumLength;
	}

	RecordingRate = SampleFrameRate;
	if (LengthInSeconds == FAnimationRecordingSettings::UnboundedMaximumLength)
	{
		// invalid length passed in, default to unbounded
		MaxFrame = UnBoundedFrameCount;
	}
	else
	{
		MaxFrame = SampleFrameRate.AsFrameNumber(LengthInSeconds);
	}
}

bool FAnimationRecorder::SetAnimCompressionScheme(UAnimBoneCompressionSettings* Settings)
{
	if (AnimationObject)
	{
		if (Settings == nullptr)
		{
			// The caller has not supplied a settings asset, use our default value
			Settings = FAnimationUtils::GetDefaultAnimationRecorderBoneCompressionSettings();
		}

		AnimationObject->BoneCompressionSettings = Settings;
		return true;
	}

	return false;
}

void FAnimationRecorder::SetCurrentFrameTimeGetter(TFunction<TOptional<FQualifiedFrameTime>()> InGetCurrentTimeFunction)
{
	GetCurrentFrameTimeFunction = MoveTemp(InGetCurrentTimeFunction);
}

// Internal. Pops up a dialog to get saved asset path
static bool PromptUserForAssetDetails(FString& AssetPath, FString& AssetName, FFrameRate& OutSampleRate, float& OutMaximumDuration)
{
	TSharedRef<SCreateAnimationDlg> NewAnimDlg = SNew(SCreateAnimationDlg);
	if (NewAnimDlg->ShowModal() != EAppReturnType::Cancel)
	{
		AssetPath = NewAnimDlg->GetFullAssetPath();
		AssetName = NewAnimDlg->GetAssetName();
		OutMaximumDuration = NewAnimDlg->GetRecordingParameters()->GetRecordingDurationSeconds();
		OutSampleRate = NewAnimDlg->GetRecordingParameters()->GetRecordingFrameRate();
		return true;
	}

	return false;
}

bool FAnimationRecorder::TriggerRecordAnimation(USkeletalMeshComponent* Component)
{
	FString AssetPath;
	FString AssetName;

	FFrameRate SampleRate;
	float MaximumLength;

	if (!Component || !Component->GetSkeletalMeshAsset() || !Component->GetSkeletalMeshAsset()->GetSkeleton())
	{
		return false;
	}

	// ask for path
	if (PromptUserForAssetDetails(AssetPath, AssetName, SampleRate, MaximumLength))
	{
		SetSampleRateAndLength(SampleRate, MaximumLength);
		return TriggerRecordAnimation(Component, AssetPath, AssetName);
	}

	return false;
}

bool FAnimationRecorder::TriggerRecordAnimation(USkeletalMeshComponent* Component, const FString& InAssetPath, const FString& InAssetName)
{
	if (!Component || !Component->GetSkeletalMeshAsset() || !Component->GetSkeletalMeshAsset()->GetSkeleton())
	{
		return false;
	}

	// create the asset
	FText InvalidPathReason;
	bool const bValidPackageName = FPackageName::IsValidLongPackageName(InAssetPath, false, &InvalidPathReason);
	if (bValidPackageName == false)
	{
		UE_LOGF(LogAnimation, Log, "%ls is an invalid asset path, prompting user for new asset path. Reason: %ls", *InAssetPath, *InvalidPathReason.ToString());
	}

	FString ValidatedAssetPath = InAssetPath;
	FString ValidatedAssetName = InAssetName;

	UObject* Parent = bValidPackageName ? CreatePackage( *ValidatedAssetPath) : nullptr;
	if (Parent == nullptr)
	{
		// bad or no path passed in, do the popup
		FFrameRate SampleRate;
		float MaximumLength;

		if (PromptUserForAssetDetails(ValidatedAssetPath, ValidatedAssetName, SampleRate, MaximumLength) == false)
		{
			return false;
		}

		SetSampleRateAndLength(SampleRate, MaximumLength);
		
		Parent = CreatePackage( *ValidatedAssetPath);	
	}

	UObject* const Object = LoadObject<UObject>(Parent, *ValidatedAssetName, nullptr, LOAD_Quiet, nullptr);
	// if object with same name exists, warn user
	if (Object)
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_AssetExist", "Asset with same name exists. Can't overwrite another asset"));
		return false;		// failed
	}

	// If not, create new one now. (also means we should not transact the recording)
	const bool bExistingAnimSequence = FindObject<UAnimSequence>(Parent, *ValidatedAssetName) != nullptr;
	bTransactRecording &= bExistingAnimSequence;

	UAnimSequence* const NewSeq = NewObject<UAnimSequence>(Parent, *ValidatedAssetName, RF_Public | RF_Standalone);
	if (NewSeq)
	{
		// set skeleton
		NewSeq->SetSkeleton(Component->GetSkeletalMeshAsset()->GetSkeleton());
		// Notify the asset registry
		FAssetRegistryModule::AssetCreated(NewSeq);
		StartRecord(Component, NewSeq);

		return true;
	}

	return false;
}

/** Helper function to get space bases depending on leader pose component */
void FAnimationRecorder::GetBoneTransforms(USkeletalMeshComponent* Component, TArray<FTransform>& BoneTransforms)
{
	const USkinnedMeshComponent* const LeaderPoseComponentInst = Component->LeaderPoseComponent.Get();
	if(LeaderPoseComponentInst)
	{
		const TArray<FTransform>& SpaceBases = LeaderPoseComponentInst->GetComponentSpaceTransforms();
		BoneTransforms.Reset(BoneTransforms.Num());
		BoneTransforms.AddUninitialized(SpaceBases.Num());
		for(int32 BoneIndex = 0; BoneIndex < SpaceBases.Num(); BoneIndex++)
		{
			if(BoneIndex < Component->GetLeaderBoneMap().Num())
			{
				int32 LeaderBoneIndex = Component->GetLeaderBoneMap()[BoneIndex];

				// If ParentBoneIndex is valid, grab matrix from LeaderPoseComponent.
				if(LeaderBoneIndex != INDEX_NONE && LeaderBoneIndex < SpaceBases.Num())
				{
					BoneTransforms[BoneIndex] = SpaceBases[LeaderBoneIndex];
				}
				else
				{
					BoneTransforms[BoneIndex] = FTransform::Identity;
				}
			}
			else
			{
				BoneTransforms[BoneIndex] = FTransform::Identity;
			}
		}
	}
	else
	{
		BoneTransforms = Component->GetComponentSpaceTransforms();
	}
}

void FAnimationRecorder::StartRecord(USkeletalMeshComponent* Component, UAnimSequence* InAnimationObject)
{
	TimePassed = 0.0;
	AnimationObject = InAnimationObject;

	if (!AnimationObject->BoneCompressionSettings)
	{
		AnimationObject->BoneCompressionSettings = FAnimationUtils::GetDefaultAnimationRecorderBoneCompressionSettings();
	}

	FAnimationRecorder::GetBoneTransforms(Component, PreviousSpacesBases);
	PreviousAnimCurves = Component->GetAnimationCurves();
	PreviousComponentToWorld = Component->GetComponentTransform();

	LastFrame = 0;

	IAnimationDataController& Controller = AnimationObject->GetController();
	Controller.SetModel(AnimationObject->GetDataModelInterface());

	Controller.OpenBracket(LOCTEXT("StartRecord_Bracket", "Starting Animation Recording"), bTransactRecording);

	Controller.InitializeModel();

	const bool bKeepNotifiesAndCurves = CVarKeepNotifyAndCurvesOnAnimationRecord->GetInt() == 0 ? false : true;
	if (bKeepNotifiesAndCurves)
	{
		Controller.RemoveAllBoneTracks(bTransactRecording);
	}
	else
	{
		AnimationObject->ResetAnimation();
	}

	if (bRemoveExcludedCurves)
	{
		TArray<FName> CurvesToRemove;
		for (const FFloatCurve& FloatCurve : AnimationObject->GetDataModel()->GetFloatCurves())
		{
			if (ShouldSkipName(FloatCurve.GetName()))
			{
				CurvesToRemove.Add(FloatCurve.GetName());
			}
		}
		for (const FName& CurveName : CurvesToRemove)
		{
			const FAnimationCurveIdentifier CurveId(CurveName, ERawCurveTrackTypes::RCT_Float);
			Controller.RemoveCurve(CurveId, bTransactRecording);
		}
	}

	RecordedCurves.Reset();
	RecordedTimes.Empty();
	
	USkeleton* AnimSkeleton = AnimationObject->GetSkeleton();
	// add all frames
	for (int32 BoneIndex=0; BoneIndex <PreviousSpacesBases.Num(); ++BoneIndex)
	{
		// verify if this bone exists in skeleton
		const int32 BoneTreeIndex = AnimSkeleton->GetSkeletonBoneIndexFromMeshBoneIndex(
			Component->LeaderPoseComponent != nullptr ?
			Component->LeaderPoseComponent->GetSkinnedAsset() :
			Component->GetSkinnedAsset(), BoneIndex);
		if (BoneTreeIndex != INDEX_NONE)
		{
			// add tracks for the bone existing
			const FName BoneTreeName = AnimSkeleton->GetReferenceSkeleton().GetBoneName(BoneTreeIndex);
			Controller.AddBoneCurve(BoneTreeName, bTransactRecording);			
			RawTracks.AddDefaulted();
		}
	}
	if (bSetRetargetSourceAsset)
	{
		AnimationObject->RetargetSource = Component->GetSkeletalMeshAsset() ? AnimSkeleton->GetRetargetSourceForMesh(Component->GetSkeletalMeshAsset()) : NAME_None;
		if (AnimationObject->RetargetSource == NAME_None)
		{
			AnimationObject->SetRetargetSourceAsset(Component->GetSkeletalMeshAsset());
			AnimationObject->UpdateRetargetSourceAssetData();
		}
	}
	else
	{
		if (AnimationObject->GetRetargetSourceAsset())
		{
			AnimationObject->SetRetargetSourceAsset(nullptr);
			AnimationObject->UpdateRetargetSourceAssetData();
		}
	}

	// record first-frame notifies
	if (Component->GetAnimInstance())
	{
		RecordNotifies(Component, Component->GetAnimInstance()->NotifyQueue.AnimNotifies, 0.0f, 0.0f);
	}

	// record the first frame
	Record(Component, PreviousComponentToWorld, PreviousSpacesBases, PreviousAnimCurves,  0);
}

void FAnimationRecorder::ProcessNotifies()
{
	if (AnimationObject == nullptr)
	{
		return;
	}

	// Get play length to clamp any active notifies against
	const float PlayLength = AnimationObject->GetDataModel()->GetPlayLength();

	// Update the end time of any notifies that are currently active
	for (FRecordedAnimNotify& RecordedNotify : RecordingAnimNotifies)
	{
		RecordedNotify.AnimNotifyEndTime = PlayLength;
		RecordedAnimNotifies.Add(RecordedNotify);
	}

	RecordingAnimNotifies.Reset();

	for (FRecordedAnimNotify& RecordedNotify : RecordedAnimNotifies)
	{
		RecordedNotify.NewNotifyEvent.SetTime(RecordedNotify.AnimNotifyStartTime);
		RecordedNotify.NewNotifyEvent.TriggerTimeOffset = 0.0f;
		RecordedNotify.NewNotifyEvent.EndTriggerTimeOffset = 0.0f;

		if (RecordedNotify.OriginalNotifyEvent->GetDuration() > 0.f)
		{
			RecordedNotify.NewNotifyEvent.SetDuration(RecordedNotify.AnimNotifyEndTime - RecordedNotify.AnimNotifyStartTime);
		}

		// see if we need to create a new notify
		if (RecordedNotify.OriginalNotifyEvent->Notify)
		{
			UAnimNotify** FoundNotify = UniqueNotifies.Find(RecordedNotify.OriginalNotifyEvent->Notify);
			if (FoundNotify == nullptr)
			{
				RecordedNotify.NewNotifyEvent.Notify = Cast<UAnimNotify>(StaticDuplicateObject(RecordedNotify.NewNotifyEvent.Notify, AnimationObject));
				UniqueNotifies.Add(RecordedNotify.OriginalNotifyEvent->Notify, RecordedNotify.NewNotifyEvent.Notify);
			}
			else
			{
				RecordedNotify.NewNotifyEvent.Notify = *FoundNotify;
			}
		}

		// see if we need to create a new notify state
		if (RecordedNotify.OriginalNotifyEvent->NotifyStateClass)
		{
			UAnimNotifyState** FoundNotifyState = UniqueNotifyStates.Find(RecordedNotify.OriginalNotifyEvent->NotifyStateClass);
			if (FoundNotifyState == nullptr)
			{
				RecordedNotify.NewNotifyEvent.NotifyStateClass = Cast<UAnimNotifyState>(StaticDuplicateObject(RecordedNotify.NewNotifyEvent.NotifyStateClass, AnimationObject));
				UniqueNotifyStates.Add(RecordedNotify.OriginalNotifyEvent->NotifyStateClass, RecordedNotify.NewNotifyEvent.NotifyStateClass);
			}
			else
			{
				RecordedNotify.NewNotifyEvent.NotifyStateClass = *FoundNotifyState;
			}
		}

		AnimationObject->Notifies.Add(RecordedNotify.NewNotifyEvent);
	}

	// build notify tracks - first find how many tracks we want
	for (FAnimNotifyEvent& Event : AnimationObject->Notifies)
	{
		if (Event.TrackIndex >= AnimationObject->AnimNotifyTracks.Num())
		{
			AnimationObject->AnimNotifyTracks.SetNum(Event.TrackIndex + 1);

			// remake track names to create a nice sequence
			const int32 TrackNum = AnimationObject->AnimNotifyTracks.Num();
			for (int32 TrackIndex = 0; TrackIndex < TrackNum; ++TrackIndex)
			{
				FAnimNotifyTrack& Track = AnimationObject->AnimNotifyTracks[TrackIndex];
				Track.TrackName = *FString::FromInt(TrackIndex + 1);
			}
		}
	}

	// now build tracks
	for (int32 EventIndex = 0; EventIndex < AnimationObject->Notifies.Num(); ++EventIndex)
	{
		FAnimNotifyEvent& Event = AnimationObject->Notifies[EventIndex];
		AnimationObject->AnimNotifyTracks[Event.TrackIndex].Notifies.Add(&AnimationObject->Notifies[EventIndex]);
	}
}

bool FAnimationRecorder::ShouldSkipName(const FName& InName) const
{
	bool bShouldSkipName = false;
			
	for (const FString& ExcludeAnimationName : ExcludeAnimationNames)
	{
		if (InName.ToString().Contains(ExcludeAnimationName))
		{
			bShouldSkipName = true;
			break;
		}
	}

	if (IncludeAnimationNames.Num() != 0)
	{
		bShouldSkipName = true;
		for (const FString& IncludeAnimationName : IncludeAnimationNames)
		{
			if (InName.ToString().Contains(IncludeAnimationName))
			{
				bShouldSkipName = false;
				break;
			}
		}
	}

	return bShouldSkipName;
}

UAnimSequence* FAnimationRecorder::StopRecord(bool bShowMessage)
{
	double StartTime, ElapsedTime = 0;

	if (IsValid(AnimationObject) && !AnimationObject->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
	{
		IAnimationDataController& Controller = AnimationObject->GetController();
		int32 NumKeys = LastFrame.Value + 1;

		//Set Interpolation type (Step or Linear), doesn't look like there is a controller for this.
		AnimationObject->Interpolation = Interpolation;

		// can't use TimePassed. That is just total time that has been passed, not necessarily match with frame count
		Controller.SetFrameRate(RecordingRate, bTransactRecording);
		Controller.SetNumberOfFrames( FMath::Max(LastFrame.Value,1), bTransactRecording);

		ProcessNotifies();

		// post-process applies compression etc.
		// @todo figure out why removing redundant keys is inconsistent

		// add to real curve data 
		if (RecordedCurves.Num() == NumKeys)
		{
			// Accumulate active curve names over recording duration
			TSet<FName> AllCurveNames;
			for(FBlendedHeapCurve& Curve : RecordedCurves)
			{
				Curve.ForEachElement([this, &AllCurveNames](const UE::Anim::FCurveElement& InElement)
				{
					// Skip curves which type is disabled in the recorder settings
					const bool bMorphTarget = EnumHasAnyFlags(InElement.Flags, UE::Anim::ECurveElementFlags::MorphTarget);
					const bool bMaterial = EnumHasAnyFlags(InElement.Flags, UE::Anim::ECurveElementFlags::Material);
					const bool bShouldSkipName = ShouldSkipName(InElement.Name);
					const bool bAttributeCurve = !bMorphTarget && !bMaterial;
					bool bSkipCurve = (bMorphTarget && !bRecordMorphTargets) ||
											(bAttributeCurve && !bRecordAttributeCurves) ||
											(bMaterial && !bRecordMaterialCurves) ||
											bShouldSkipName;
					if(!bSkipCurve)
					{
						AllCurveNames.Add(InElement.Name);
					}
				});
			}
			

			StartTime = FPlatformTime::Seconds();
			USkeleton* SkeletonObj = AnimationObject->GetSkeleton();
			for (FName CurveName : AllCurveNames)
			{
				const FFloatCurve* FloatCurveData = nullptr;

				TArray<float> TimesToRecord;
				TArray<float> ValuesToRecord;
				TimesToRecord.SetNum(NumKeys);
				ValuesToRecord.SetNum(NumKeys);

				bool bSeenThisCurve = false;
				int32 WriteIndex = 0;
				for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
				{
					const float TimeToRecord = RecordingRate.AsSeconds(KeyIndex);
					bool bIsCurveValid = false;
					float CurCurveValue = RecordedCurves[KeyIndex].Get(CurveName, bIsCurveValid);
					if(bIsCurveValid)
					{
						if (!bSeenThisCurve)
						{
							bSeenThisCurve = true;

							// add one and save the cache
							// give default curve flag for recording 
							const FAnimationCurveIdentifier CurveId(CurveName, ERawCurveTrackTypes::RCT_Float);
							Controller.AddCurve(CurveId, AACF_DefaultCurve, bTransactRecording);
							FloatCurveData = AnimationObject->GetDataModel()->FindFloatCurve(CurveId);
						}

						if (FloatCurveData)
						{
							TimesToRecord[WriteIndex] = TimeToRecord;
							ValuesToRecord[WriteIndex] = CurCurveValue;

							++WriteIndex;
						}
					}
				}

				// Fill all the curve data at once
				if (FloatCurveData)
				{
					//since default is true to check nonzero values we avoid the if state to test and just
					//always check
					bool bHasNonZeroValue = false;
					TArray<FRichCurveKey> Keys;
					for (int32 Index = 0; Index < WriteIndex; ++Index)
					{
						float Value = ValuesToRecord[Index];
						if (bHasNonZeroValue == false)
						{
							if (FMath::IsNearlyZero(Value, UE_KINDA_SMALL_NUMBER) == false)
							{
								bHasNonZeroValue = true;
							}
						}
						FRichCurveKey Key(TimesToRecord[Index], Value);
						Key.InterpMode = InterpMode;
						Key.TangentMode = TangentMode;
						Keys.Add(Key);
					}
					const FAnimationCurveIdentifier CurveId(FloatCurveData->GetName(), ERawCurveTrackTypes::RCT_Float);
					if(bSkipCurvesWithZeroValue == false || bHasNonZeroValue)
					{ 
						Controller.SetCurveKeys(CurveId, Keys, bTransactRecording);
					}
					else
					{
						Controller.RemoveCurve(CurveId, bTransactRecording);
					}
				}
			}

			ElapsedTime = FPlatformTime::Seconds() - StartTime;
			UE_LOGF(LogAnimation, Log, "Animation Recorder set keys in %0.02f seconds", ElapsedTime);
		}

		// Populate bone tracks
		TArray<FName> TrackNames;
		AnimationObject->GetDataModel()->GetBoneTrackNames(TrackNames);
		for (int32 TrackIndex = 0; TrackIndex < TrackNames.Num(); ++TrackIndex)
		{
			const FRawAnimSequenceTrack& RawTrack = RawTracks[TrackIndex];
			FName BoneName = TrackNames[TrackIndex];

			bool bShouldSkipName = ShouldSkipName(BoneName);

			if (!bShouldSkipName)
			{
				Controller.SetBoneTrackKeys(BoneName, RawTrack.PosKeys, RawTrack.RotKeys, RawTrack.ScaleKeys, bTransactRecording);
			}
			else
			{
				TArray<FVector3f> SinglePosKey;
				SinglePosKey.Add(RawTrack.PosKeys[0]);
				TArray<FQuat4f> SingleRotKey;
				SingleRotKey.Add(RawTrack.RotKeys[0]);
				TArray<FVector3f> SingleScaleKey;
				SingleScaleKey.Add(RawTrack.ScaleKeys[0]);

				Controller.SetBoneTrackKeys(BoneName, SinglePosKey, SingleRotKey, SingleScaleKey, bTransactRecording);

				UE_LOGF(LogAnimation, Log, "Animation Recorder skipping bone: %ls", *BoneName.ToString());
			}
		}

		if (bRecordTransforms == false)
		{
			Controller.RemoveAllBoneTracks(bTransactRecording);
		}

		Controller.NotifyPopulated();
		Controller.CloseBracket(bTransactRecording);

		AnimationObject->PostEditChange();
		AnimationObject->MarkPackageDirty();
		
		// save the package to disk, for convenience and so we can run this in standalone mode
		if (bAutoSaveAsset)
		{
			UPackage* const Package = AnimationObject->GetOutermost();
			FString const PackageName = Package->GetName();
			FString const PackageFileName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());
			
			StartTime = FPlatformTime::Seconds();

			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_Standalone;
			SaveArgs.SaveFlags = SAVE_NoError;
			UPackage::SavePackage(Package, nullptr, *PackageFileName, SaveArgs);

			ElapsedTime = FPlatformTime::Seconds() - StartTime;
			UE_LOGF(LogAnimation, Log, "Animation Recorder saved %ls in %0.2f seconds", *PackageName, ElapsedTime);
		}

		UAnimSequence* ReturnObject = AnimationObject;

		// notify to user
		if (bShowMessage)
		{
			const FText NotificationText = FText::Format(LOCTEXT("RecordAnimation", "'{0}' has been successfully recorded [{1} keys : {2} sec(s) @ {3} Hz]"),
				FText::FromString(AnimationObject->GetName()),
				FText::AsNumber(AnimationObject->GetDataModel()->GetNumberOfKeys()),
				FText::AsNumber(AnimationObject->GetPlayLength()),
				RecordingRate.ToPrettyText()
				);
					
			if (GIsEditor)
			{
				FNotificationInfo Info(NotificationText);
				Info.ExpireDuration = 8.0f;
				Info.bUseLargeFont = false;
				Info.Hyperlink = FSimpleDelegate::CreateLambda([=]()
				{
					TArray<UObject*> Assets;
					Assets.Add(ReturnObject);
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(Assets);
				});
				Info.HyperlinkText = FText::Format(LOCTEXT("OpenNewAnimationHyperlink", "Open {0}"), FText::FromString(AnimationObject->GetName()));
				TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
				if ( Notification.IsValid() )
				{
					Notification->SetCompletionState( SNotificationItem::CS_Success );
				}
			}

			FAssetRegistryModule::AssetCreated(AnimationObject);
		}

		AnimationObject = nullptr;
		PreviousSpacesBases.Empty();
		PreviousAnimCurves.Empty();

		return ReturnObject;
	}

	UniqueNotifies.Empty();
	UniqueNotifyStates.Empty();

	return nullptr;
}
void FAnimationRecorder::ProcessRecordedTimes(UAnimSequence* AnimSequence, USkeletalMeshComponent* SkeletalMeshComponent, const FString& HoursName, const FString& MinutesName, const FString& SecondsName, const FString& FramesName, const FString& SubFramesName, const FString& SlateName, const FString& Slate, const FTimecodeBoneMethod& TimecodeBoneMethod)
{
	FProcessRecordedTimeParams Params{
		 .HoursName = HoursName,
		 .MinutesName = MinutesName,
		 .SecondsName = SecondsName,
		 .FramesName = FramesName,
		 .SubFramesName = SubFramesName,
		 .SlateName = SlateName,
		 .Slate = Slate
	};
	ProcessRecordedTimes(AnimSequence, SkeletalMeshComponent, TimecodeBoneMethod, Params);
}

void FAnimationRecorder::ProcessRecordedTimes(UAnimSequence* AnimSequence, USkeletalMeshComponent* SkeletalMeshComponent, const FTimecodeBoneMethod& TimecodeBoneMethod, const FProcessRecordedTimeParams& InParams)
{
	if (!AnimSequence || !SkeletalMeshComponent)
	{
		return;
	}

	int32 NumKeys = LastFrame.Value + 1;
	if (RecordedTimes.Num() != NumKeys)
	{
		return;
	}

	TArray<int32> Hours, Minutes, Seconds, Frames;
	TArray<float> SubFrames;
	TArray<float> Times;

	Hours.Reserve(RecordedTimes.Num());
	Minutes.Reserve(RecordedTimes.Num());
	Seconds.Reserve(RecordedTimes.Num());
	Frames.Reserve(RecordedTimes.Num());
	SubFrames.Reserve(RecordedTimes.Num());
	Times.Reserve(RecordedTimes.Num());

	FFrameRate TCRate = RecordingRate;
	for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
	{
		const float TimeToRecord = RecordingRate.AsSeconds(KeyIndex);

		FQualifiedFrameTime RecordedTime = RecordedTimes[KeyIndex];
		FTimecode Timecode = FTimecode::FromFrameNumber(RecordedTime.Time.FrameNumber, RecordedTime.Rate);
		TCRate = RecordedTime.Rate;
		Hours.Add(Timecode.Hours);
		Minutes.Add(Timecode.Minutes);
		Seconds.Add(Timecode.Seconds);
		Frames.Add(Timecode.Frames);

		float SubFrame = RecordedTime.Time.GetSubFrame();
		SubFrames.Add(SubFrame);

		Times.Add(TimeToRecord);
	}

	Hours.Shrink();
	Minutes.Shrink();
	Seconds.Shrink();
	Frames.Shrink();
	SubFrames.Shrink();
	Times.Shrink();
	
	USkeleton* AnimSkeleton = AnimSequence->GetSkeleton();

	const USkinnedMeshComponent* const LeaderPoseComponentInst = SkeletalMeshComponent->LeaderPoseComponent.Get();
	const TArray<FTransform>* SpaceBases;
	if (LeaderPoseComponentInst)
	{
		SpaceBases = &LeaderPoseComponentInst->GetComponentSpaceTransforms();
	}
	else
	{
		SpaceBases = &SkeletalMeshComponent->GetComponentSpaceTransforms();
	}

	// String is not animatable, just add 1 slate at the first key time
	TArray<FString> Slates(&InParams.Slate, 1);
	TArray<float> SlateTimes(&Times[0], 1);

	float RecordingRateAsFloat = TCRate.AsDecimal();
	TOptional<TArray<float>> Rates;
	TOptional<TArray<float>> RateTimes;
	if (InParams.RateName)
	{
		Rates = TArray<float>(&RecordingRateAsFloat, 1);
		RateTimes = TArray<float>(&Times[0],1);
	}

	IAnimationDataController& Controller = AnimSequence->GetController();
	IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("AddTimeCodeAttributesBracket", "Adding Time Code attributes"));

	// If the user defined bone doesn't exist, fallback to writing timecodes to the root
	bool bHasUserDefinedBone = false;
	if (TimecodeBoneMethod.BoneMode == ETimecodeBoneMode::UserDefined)
	{
		for (int32 BoneIndex = 0; BoneIndex < SpaceBases->Num(); ++BoneIndex)
		{
			const int32 BoneTreeIndex = AnimSkeleton->GetSkeletonBoneIndexFromMeshBoneIndex(
				SkeletalMeshComponent->LeaderPoseComponent != nullptr ?
				SkeletalMeshComponent->LeaderPoseComponent->GetSkinnedAsset() :
				SkeletalMeshComponent->GetSkinnedAsset(), BoneIndex);
			if (BoneTreeIndex != INDEX_NONE)
			{
				FName BoneTreeName = AnimSkeleton->GetReferenceSkeleton().GetBoneName(BoneTreeIndex);
				if (BoneTreeName == TimecodeBoneMethod.BoneName)
				{
					bHasUserDefinedBone = true;
					break;
				}
			}
		}

		if (!bHasUserDefinedBone)
		{
			UE_LOGF(LogAnimation, Warning, "User defined bone name: %ls not found. Falling back to assigning timecodes to root bone", *TimecodeBoneMethod.BoneName.ToString());
		}
	}

	for (int32 BoneIndex = 0; BoneIndex < SpaceBases->Num(); ++BoneIndex)
	{
		// verify if this bone exists in skeleton
		const int32 BoneTreeIndex = AnimSkeleton->GetSkeletonBoneIndexFromMeshBoneIndex(
			SkeletalMeshComponent->LeaderPoseComponent != nullptr ?
			SkeletalMeshComponent->LeaderPoseComponent->GetSkinnedAsset() :
			SkeletalMeshComponent->GetSkinnedAsset(), BoneIndex);
		if (BoneTreeIndex != INDEX_NONE)
		{
			// add tracks for the bone existing
			FName BoneTreeName = AnimSkeleton->GetReferenceSkeleton().GetBoneName(BoneTreeIndex);

			const bool bUseThisBone = 
				TimecodeBoneMethod.BoneMode == ETimecodeBoneMode::All ||
				(TimecodeBoneMethod.BoneMode == ETimecodeBoneMode::Root && BoneIndex == 0) ||
				(TimecodeBoneMethod.BoneMode == ETimecodeBoneMode::UserDefined && BoneTreeName == TimecodeBoneMethod.BoneName) ||
				(bHasUserDefinedBone == false && BoneIndex == 0);

			if (!bUseThisBone)
			{
				continue;
			}

			UE::Anim::AddTypedCustomAttribute<FIntegerAnimationAttribute, int32>(FName(*InParams.HoursName), BoneTreeName, AnimSequence, MakeArrayView(Times), MakeArrayView(Hours));
			UE::Anim::AddTypedCustomAttribute<FIntegerAnimationAttribute, int32>(FName(*InParams.MinutesName), BoneTreeName, AnimSequence, MakeArrayView(Times), MakeArrayView(Minutes));
			UE::Anim::AddTypedCustomAttribute<FIntegerAnimationAttribute, int32>(FName(*InParams.SecondsName), BoneTreeName, AnimSequence, MakeArrayView(Times), MakeArrayView(Seconds));
			UE::Anim::AddTypedCustomAttribute<FIntegerAnimationAttribute, int32>(FName(*InParams.FramesName), BoneTreeName, AnimSequence, MakeArrayView(Times), MakeArrayView(Frames));

			UE::Anim::AddTypedCustomAttribute<FFloatAnimationAttribute, float>(FName(*InParams.SubFramesName), BoneTreeName, AnimSequence, MakeArrayView(Times), MakeArrayView(SubFrames));

			if (Rates && RateTimes)
			{
				UE::Anim::AddTypedCustomAttribute<FFloatAnimationAttribute, float>(FName(*(InParams.RateName.GetValue())), BoneTreeName, AnimSequence, MakeArrayView(*RateTimes), MakeArrayView(*Rates));
			}
			UE::Anim::AddTypedCustomAttribute<FStringAnimationAttribute, FString>(FName(*InParams.SlateName), BoneTreeName, AnimSequence, MakeArrayView(SlateTimes), MakeArrayView(Slates));

		}
	}
}

void FAnimationRecorder::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (AnimationObject)
	{
		Collector.AddReferencedObject(AnimationObject);
	}
}

void FAnimationRecorder::UpdateRecord(USkeletalMeshComponent* Component, float DeltaTime)
{
	// if no animation object, return
	if (!AnimationObject || !Component)
	{
		return;
	}

	// no sim time, no record
	if (DeltaTime <= 0.f)
	{
		return;
	}

	// Take Recorder will turn this off, not sure if it's needed for persona animation recording or not.
	if (bCheckDeltaTimeAtBeginning)
	{
		// in-editor we can get a long frame update because of the modal dialog used to pick paths
		if (DeltaTime > RecordingRate.AsInterval() && (LastFrame == 0 || LastFrame == 1))
		{
			DeltaTime = RecordingRate.AsInterval();
		}
	}

	float const PreviousTimePassed = TimePassed;
	TimePassed += DeltaTime;

	// time passed has been updated
	// now find what frames we need to update
	int32 FramesRecorded = LastFrame.Value;
	int32 FramesToRecord = RecordingRate.AsFrameNumber(TimePassed).Value;

	// notifies need to be done regardless of sample rate
	if (Component->GetAnimInstance())
	{
		RecordNotifies(Component, Component->GetAnimInstance()->NotifyQueue.AnimNotifies, DeltaTime, TimePassed);
	}

	TArray<FTransform> SpaceBases;
	FAnimationRecorder::GetBoneTransforms(Component, SpaceBases);

	if (FramesRecorded < FramesToRecord)
	{
		const FBlendedHeapCurve& AnimCurves = Component->GetAnimationCurves();

		if (SpaceBases.Num() != PreviousSpacesBases.Num())
		{
			UE_LOGF(LogAnimation, Log, "Current Num of Spaces %d don't match with the previous number %d so we are stopping recording", SpaceBases.Num(),PreviousSpacesBases.Num());
			StopRecord(true);
			return;
		}
		TArray<FTransform> BlendedSpaceBases;
		BlendedSpaceBases.AddZeroed(SpaceBases.Num());

		UE_LOGF(LogAnimation, Log, "DeltaTime : %0.2f, Current Frame Count : %d, Frames To Record : %d, TimePassed : %0.2f", DeltaTime
			, FramesRecorded, FramesToRecord, TimePassed);

		// if we need to record frame
		while (FramesToRecord > FramesRecorded)
		{
			// find what frames we need to record
			// convert to time
			const float CurrentTime = RecordingRate.AsSeconds(FramesRecorded + 1);
			float BlendAlpha = (CurrentTime - PreviousTimePassed) / DeltaTime;

			UE_LOGF(LogAnimation, Log, "Current Frame Count : %d, BlendAlpha : %0.2f", FramesRecorded + 1, BlendAlpha);

			// for now we just concern component space, not skeleton space
			for (int32 BoneIndex = 0; BoneIndex<SpaceBases.Num(); ++BoneIndex)
			{
				BlendedSpaceBases[BoneIndex].Blend(PreviousSpacesBases[BoneIndex], SpaceBases[BoneIndex], BlendAlpha);
			}

			FTransform BlendedComponentToWorld;
			BlendedComponentToWorld.Blend(PreviousComponentToWorld, Component->GetComponentTransform(), BlendAlpha);

			FBlendedHeapCurve BlendedCurve;
			BlendedCurve.Lerp(PreviousAnimCurves, AnimCurves, BlendAlpha);

			if (!Record(Component, BlendedComponentToWorld, BlendedSpaceBases, BlendedCurve, FramesRecorded + 1))
			{
				StopRecord(true);
				return;
			}
			++FramesRecorded;
		}
	}

	//save to current transform
	PreviousSpacesBases = SpaceBases;
	PreviousAnimCurves = Component->GetAnimationCurves();
	PreviousComponentToWorld = Component->GetComponentTransform();

	// if we passed MaxFrame, just stop it
	if (MaxFrame != UnBoundedFrameCount && FramesRecorded >= MaxFrame)
	{
		UE_LOGF(LogAnimation, Log, "Animation Recording exceeds the time limited (%f seconds). Stopping recording animation... ", RecordingRate.AsSeconds(MaxFrame));
		FAnimationRecorderManager::Get().StopRecordingAnimation(Component, true);
		return;
	}
}

bool FAnimationRecorder::Record(USkeletalMeshComponent* Component, FTransform const& ComponentToWorld, const TArray<FTransform>& SpacesBases, const FBlendedHeapCurve& AnimationCurves, int32 FrameToAdd)
{
	if (ensure(AnimationObject))
	{
		IAnimationDataController& Controller = AnimationObject->GetController();
		USkinnedAsset* SkinnedAsset =
			Component->LeaderPoseComponent != nullptr ?
			Component->LeaderPoseComponent->GetSkinnedAsset() :
			Component->GetSkinnedAsset();

		TArray<FName> TrackNames;
		const IAnimationDataModel* DataModel = AnimationObject->GetDataModel();
		DataModel->GetBoneTrackNames(TrackNames);

		if (FrameToAdd == 0)
		{
			// Find the root bone & store its transform
			SkeletonRootIndex = INDEX_NONE;
			USkeleton* AnimSkeleton = AnimationObject->GetSkeleton();

			for (const FName& TrackName : TrackNames)
			{
				// verify if this bone exists in skeleton
				const int32 BoneTreeIndex = AnimSkeleton->GetReferenceSkeleton().FindBoneIndex(TrackName);
				if (BoneTreeIndex != INDEX_NONE)
				{
					const int32 BoneIndex = AnimSkeleton->GetMeshBoneIndexFromSkeletonBoneIndex(SkinnedAsset, BoneTreeIndex);
					const int32 ParentIndex = SkinnedAsset->GetRefSkeleton().GetParentIndex(BoneIndex);
					const FTransform LocalTransform = SpacesBases[BoneIndex];
					if (ParentIndex == INDEX_NONE)
					{
						if (bRemoveRootTransform && TrackNames.Num() > 1)
						{
							// Store initial root transform.
							// We remove the initial transform of the root bone and transform root's children
							// to remove any offset. We need to do this for sequence recording in particular
							// as we use root motion to build transform tracks that properly sync with
							// animation keyframes. If we have a transformed root bone then the assumptions 
							// we make about root motion use are incorrect.
							// NEW. But we don't do this if there is just one root bone. This has come up with recording
							// single bone props and cameras.
							InitialRootTransform = LocalTransform;
							InvInitialRootTransform = LocalTransform.Inverse();
						}
						else
						{
							InitialRootTransform = InvInitialRootTransform = FTransform::Identity;
						}
						SkeletonRootIndex = BoneIndex;
						break;
					}
				}
			}
		}

		FSerializedAnimation  SerializedAnimation;
		USkeleton* AnimSkeleton = AnimationObject->GetSkeleton();

		for (int32 TrackIndex = 0; TrackIndex < TrackNames.Num(); ++TrackIndex)
		{
			const FName& TrackName = TrackNames[TrackIndex];
			FRawAnimSequenceTrack& RawTrack = RawTracks[TrackIndex];

			// verify if this bone exists in skeleton
			const int32 BoneTreeIndex = AnimSkeleton->GetReferenceSkeleton().FindBoneIndex(TrackName);
			if (BoneTreeIndex != INDEX_NONE)
			{
				const int32 BoneIndex = AnimSkeleton->GetMeshBoneIndexFromSkeletonBoneIndex(SkinnedAsset, BoneTreeIndex);
				const int32 ParentIndex = SkinnedAsset->GetRefSkeleton().GetParentIndex(BoneIndex);

				if (bRecordTransforms)
				{
					FTransform LocalTransform = SpacesBases[BoneIndex];
					if ( ParentIndex != INDEX_NONE )
					{
						LocalTransform.SetToRelativeTransform(SpacesBases[ParentIndex]);
					}
					// if record local to world, we'd like to consider component to world to be in root
					else
					{
						if (bRecordLocalToWorld)
						{
							LocalTransform *= ComponentToWorld;
						}
					}

					RawTrack.PosKeys.Add(FVector3f(LocalTransform.GetTranslation()));
					RawTrack.RotKeys.Add(FQuat4f(LocalTransform.GetRotation()));
					RawTrack.ScaleKeys.Add(FVector3f(LocalTransform.GetScale3D()));  
					if (AnimationSerializer)
					{
						SerializedAnimation.AddTransform(TrackIndex, LocalTransform);
					}
					// verification
					if (FrameToAdd != RawTrack.PosKeys.Num() - 1)
					{
						UE_LOGF(LogAnimation, Warning, "Mismatch in animation frames. Trying to record frame: %d, but only: %d frame(s) exist. Changing skeleton while recording is not supported.", FrameToAdd, RawTrack.PosKeys.Num());
						return false;
					}
				}
				else
				{
					if (FrameToAdd == 0)
					{
						const FTransform RefPose = Component->GetSkeletalMeshAsset()->GetRefSkeleton().GetRefBonePose()[BoneIndex];
						RawTrack.PosKeys.Add(FVector3f(RefPose.GetTranslation()));
						RawTrack.RotKeys.Add(FQuat4f(RefPose.GetRotation()));
						RawTrack.ScaleKeys.Add(FVector3f(RefPose.GetScale3D()));
					}
				}
			}
		}

		TOptional<FQualifiedFrameTime> CurrentTime;
		
		// When set, use the custom time getter. Otherwise fall back to FApp::GetCurrentFrameTime.
		if (GetCurrentFrameTimeFunction)
		{
			CurrentTime = GetCurrentFrameTimeFunction();
		}
		else
		{
			CurrentTime = FApp::GetCurrentFrameTime();
		}

		if (CurrentTime)
		{
			FQualifiedFrameTime TimeToRecord = *CurrentTime;
			if (TimeToRecord.Time.GetSubFrame() == 0)
			{
				FQualifiedFrameTime FrameTimeAsQualified(FrameToAdd, RecordingRate);
				FFrameTime FrameTimeInTCRate = FrameTimeAsQualified.ConvertTo(TimeToRecord.Rate);
				FFrameTime NewFrameTimeWithSubFrame(TimeToRecord.Time.GetFrame(), FrameTimeInTCRate.GetSubFrame());
				TimeToRecord = FQualifiedFrameTime(NewFrameTimeWithSubFrame, CurrentTime->Rate);
			}
			RecordedTimes.Add(TimeToRecord);
		}
		else
		{
			RecordedTimes.Add(FQualifiedFrameTime());
		}

		if (AnimationSerializer)
		{
			AnimationSerializer->WriteFrameData(AnimationSerializer->FramesWritten, SerializedAnimation);
		}
		// each RecordedCurves contains all elements
		const bool bRecordCurves = bRecordMorphTargets || bRecordAttributeCurves || bRecordMaterialCurves; 
		if (bRecordCurves)
		{
			RecordedCurves.Emplace(AnimationCurves);
		}

		LastFrame = FrameToAdd;
	}

	return true;
}

void FAnimationRecorder::RecordNotifies(USkeletalMeshComponent* Component, const TArray<FAnimNotifyEventReference>& AnimNotifies, float DeltaTime, float RecordTime)
{
	if (ensure(AnimationObject))
	{
		// flag notifies as possibly unused this frame
		for (FRecordedAnimNotify& RecordingNotify : RecordingAnimNotifies)
		{
			RecordingNotify.bWasActive = false;
		}

		int32 AddedThisFrame = 0;
		for(const FAnimNotifyEventReference& NotifyEventRef : AnimNotifies)
		{
			if(const FAnimNotifyEvent* NotifyEvent = NotifyEventRef.GetNotify())
			{
				if (NotifyEvent->GetDuration() <= 0.f)
				{
					// If the duration of this notify <= 0.0 we just store it as recorded
					RecordedAnimNotifies.Emplace(*NotifyEvent, NotifyEvent, RecordTime, RecordTime);
				}
				else
				{
					// Check to see if the notify is already active
					bool bAlreadyActive = false;
					for (FRecordedAnimNotify& RecordingNotify : RecordingAnimNotifies)
					{
						if (NotifyEvent == RecordingNotify.OriginalNotifyEvent)
						{
							bAlreadyActive = true;
							RecordingNotify.bWasActive = true;
							break;
						}
					}

					// Add this 'new' notify
					if (!bAlreadyActive)
					{
						RecordingAnimNotifies.Emplace(*NotifyEvent, NotifyEvent, RecordTime, RecordTime);
					}
				}

				AddedThisFrame++;
			}
		}

		for (int32 RecordingAnimNotifyIndex = 0; RecordingAnimNotifyIndex < RecordingAnimNotifies.Num(); ++RecordingAnimNotifyIndex)
		{
			FRecordedAnimNotify& RecordingNotify = RecordingAnimNotifies[RecordingAnimNotifyIndex];

			// If a notify wasnt active this frame, commit it as 'recorded'
			if (!RecordingNotify.bWasActive)
			{
				RecordingNotify.AnimNotifyEndTime = RecordTime;
				RecordedAnimNotifies.Add(RecordingNotify);
				RecordingAnimNotifies.RemoveAtSwap(RecordingAnimNotifyIndex);
			}
		}

		UE_LOGF(LogAnimation, Log, "Added notifies : %d", AddedThisFrame);
	}
}

void FAnimationRecorderManager::Tick(float DeltaTime)
{
	// Not range-based as instance can delete upon update
	for (int32 i = 0; i < RecorderInstances.Num(); ++i)
	{
		RecorderInstances[i].Update(DeltaTime);
	}
}

void FAnimationRecorderManager::Tick(USkeletalMeshComponent* Component, float DeltaTime)
{
	for (auto& Inst : RecorderInstances)
	{
		if(Inst.SkelComp == Component)
		{
			Inst.Update(DeltaTime);
		}
	}
}

FAnimationRecorderManager::FAnimationRecorderManager()
{
}

FAnimationRecorderManager::~FAnimationRecorderManager()
{
}

FAnimationRecorderManager& FAnimationRecorderManager::Get()
{
	static FAnimationRecorderManager AnimRecorderManager;
	return AnimRecorderManager;
}


FAnimRecorderInstance::FAnimRecorderInstance()
	: SkelComp(nullptr)
	, Recorder(nullptr)
	, CachedVisibilityBasedAnimTickOption(EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones)
	, bCachedEnableUpdateRateOptimizations(false)
{
}

void FAnimRecorderInstance::Init(USkeletalMeshComponent* InComponent, const FString& InAssetPath, const FString& InAssetName, const FAnimationRecordingSettings& Settings)
{
	AssetPath = InAssetPath;
	AssetName = InAssetName;

	InitInternal(InComponent, Settings);
}

void FAnimRecorderInstance::Init(USkeletalMeshComponent* InComponent, UAnimSequence* InSequence, FAnimationSerializer *InAnimationSerializer, const FAnimationRecordingSettings& Settings)
{
	Sequence = InSequence;
	InitInternal(InComponent, Settings,InAnimationSerializer);
}

void FAnimRecorderInstance::InitInternal(USkeletalMeshComponent* InComponent, const FAnimationRecordingSettings& Settings, FAnimationSerializer *InAnimationSerializer)
{
	SkelComp = InComponent;
	Recorder = MakeShareable(new FAnimationRecorder());
	Recorder->SetSampleRateAndLength(Settings.SampleFrameRate, Settings.Length);
	Recorder->bSetRetargetSourceAsset = Settings.bSetRetargetSourceAsset;
	Recorder->bRecordLocalToWorld = Settings.bRecordInWorldSpace;
	Recorder->Interpolation = Settings.Interpolation;
	Recorder->InterpMode = Settings.InterpMode;
	Recorder->TangentMode = Settings.TangentMode;
	Recorder->bAutoSaveAsset = Settings.bAutoSaveAsset;
	Recorder->bRemoveRootTransform = Settings.bRemoveRootAnimation;
	Recorder->bCheckDeltaTimeAtBeginning = Settings.bCheckDeltaTimeAtBeginning;
	Recorder->AnimationSerializer = InAnimationSerializer;
	Recorder->bRecordTransforms = Settings.bRecordTransforms;
	Recorder->bRecordMorphTargets = Settings.bRecordMorphTargets;
	Recorder->bRecordAttributeCurves = Settings.bRecordAttributeCurves;
	Recorder->bRecordMaterialCurves = Settings.bRecordMaterialCurves;
	Recorder->bSkipCurvesWithZeroValue = Settings.bSkipCurvesWithZeroValue;
	Recorder->bTransactRecording = Settings.bTransactRecording;
	Recorder->IncludeAnimationNames = Settings.IncludeAnimationNames;
	Recorder->ExcludeAnimationNames = Settings.ExcludeAnimationNames;
	Recorder->bRemoveExcludedCurves = Settings.bRemoveExcludedCurves;

	if (InComponent)
	{
		CachedSkelCompForcedLodModel = InComponent->GetForcedLOD();
		InComponent->SetForcedLOD(1);

		// turn off URO and make sure we always update even if out of view
		bCachedEnableUpdateRateOptimizations = InComponent->bEnableUpdateRateOptimizations;
		CachedVisibilityBasedAnimTickOption = InComponent->VisibilityBasedAnimTickOption;

		InComponent->bEnableUpdateRateOptimizations = false;
		InComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	}
}

FAnimRecorderInstance::~FAnimRecorderInstance()
{
}

bool FAnimRecorderInstance::BeginRecording()
{
	if (SkelComp.IsValid() == false)
	{
		UE_LOGF(LogAnimation, Log, "Animation Recorder:  Begin Recording: SkelMeshComp not Valid, No Recording will occur.");
		return false;
	}
	if (Recorder.IsValid())
	{
		if (Sequence.IsValid())
		{
			Recorder->StartRecord(SkelComp.Get(), Sequence.Get());
			return true;
		}
		else
		{
			return Recorder->TriggerRecordAnimation(SkelComp.Get(), AssetPath, AssetName);
		}
	}

	UE_LOGF(LogAnimation, Log, "Animation Recorder: Begin Recording: Recorder not Valid, No Recording will occur.");
	return false;
}

void FAnimRecorderInstance::Update(float DeltaTime)
{
	if (SkelComp.IsValid() == false)
	{
		UE_LOGF(LogAnimation, Log, "Animation Recorder:  Update: SkelMeshComp not Valid, No Recording will occur.");
		return;
	}
	if (Recorder.IsValid())
	{
		Recorder->UpdateRecord(SkelComp.Get(), DeltaTime);
	}
	else
	{
		UE_LOGF(LogAnimation, Log, "Animation Recorder:  Update: Recoder not Valid, No Recording will occur.");
	}
}
void FAnimRecorderInstance::FinishRecording(bool bShowMessage)
{
	const FText FinishRecordingAnimationSlowTask = LOCTEXT("FinishRecordingAnimationSlowTask", "Finalizing recorded animation");
	if (Recorder.IsValid())
	{
		Recorder->StopRecord(bShowMessage);
	}

	if (SkelComp.IsValid())
	{
		// restore force lod setting
		SkelComp->SetForcedLOD(CachedSkelCompForcedLodModel);

		// restore update flags
		SkelComp->bEnableUpdateRateOptimizations = bCachedEnableUpdateRateOptimizations;
		SkelComp->VisibilityBasedAnimTickOption = CachedVisibilityBasedAnimTickOption;
	}
}

void FAnimRecorderInstance::ProcessRecordedTimes(UAnimSequence* AnimSequence, USkeletalMeshComponent* SkeletalMeshComponent, const FString& HoursName, const FString& MinutesName, const FString& SecondsName, const FString& FramesName, const FString& SubFramesName, const FString& SlateName, const FString& Slate, const FTimecodeBoneMethod& TimecodeBoneMethod)
{
	if (Recorder.IsValid())
	{
		FProcessRecordedTimeParams Params{
		 .HoursName = HoursName,
		 .MinutesName = MinutesName,
		 .SecondsName = SecondsName,
		 .FramesName = FramesName,
		 .SubFramesName = SubFramesName,
		 .SlateName = SlateName,
		 .Slate = Slate
		};
		Recorder->ProcessRecordedTimes(AnimSequence, SkeletalMeshComponent, TimecodeBoneMethod, Params);
	}
}

void FAnimRecorderInstance::ProcessRecordedTimes(UAnimSequence* AnimSequence, USkeletalMeshComponent* SkeletalMeshComponent, const FTimecodeBoneMethod& TimecodeBoneMethod, const FProcessRecordedTimeParams& InParams)
{
	if (Recorder.IsValid())
	{
		Recorder->ProcessRecordedTimes(AnimSequence, SkeletalMeshComponent, TimecodeBoneMethod, InParams);
	}
}

bool FAnimationRecorderManager::RecordAnimation(USkeletalMeshComponent* Component, const FString& AssetPath, const FString& AssetName, const FAnimationRecordingSettings& Settings)
{
	if (Component)
	{
		FAnimRecorderInstance NewInst;
		NewInst.Init(Component, AssetPath, AssetName, Settings);
		bool const bSuccess = NewInst.BeginRecording();
		if (bSuccess)
		{
			RecorderInstances.Add(NewInst);
		}

	#if WITH_EDITOR
			// if recording via PIE, be sure to stop recording cleanly when PIE ends
			UWorld const* const World = Component->GetWorld();
			if (World && World->IsPlayInEditor())
			{
				FEditorDelegates::EndPIE.AddRaw(this, &FAnimationRecorderManager::HandleEndPIE);
			}
	#endif

		return bSuccess;
	}

	return false;
}

bool FAnimationRecorderManager::RecordAnimation(USkeletalMeshComponent* Component, UAnimSequence* Sequence, const FAnimationRecordingSettings& Settings)
{
	if (Component)
	{
		FAnimRecorderInstance NewInst;
		NewInst.Init(Component, Sequence, nullptr, Settings);
		bool const bSuccess = NewInst.BeginRecording();
		if (bSuccess)
		{
			RecorderInstances.Add(NewInst);
		}

	#if WITH_EDITOR
			// if recording via PIE, be sure to stop recording cleanly when PIE ends
			UWorld const* const World = Component->GetWorld();
			if (World && World->IsPlayInEditor())
			{
				FEditorDelegates::EndPIE.AddRaw(this, &FAnimationRecorderManager::HandleEndPIE);
			}
	#endif

		return bSuccess;
	}

	return false;
}

bool FAnimationRecorderManager::RecordAnimation(USkeletalMeshComponent* Component, UAnimSequence* Sequence, FAnimationSerializer* InSerializer, const FAnimationRecordingSettings& Settings)
{
	if (Component)
	{
		FAnimRecorderInstance NewInst;
		NewInst.Init(Component, Sequence, InSerializer, Settings);
		bool const bSuccess = NewInst.BeginRecording();
		if (bSuccess)
		{
			RecorderInstances.Add(NewInst);
		}

#if WITH_EDITOR
		// if recording via PIE, be sure to stop recording cleanly when PIE ends
		UWorld const* const World = Component->GetWorld();
		if (World && World->IsPlayInEditor())
		{
			FEditorDelegates::EndPIE.AddRaw(this, &FAnimationRecorderManager::HandleEndPIE);
		}
#endif

		return bSuccess;
	}

	return false;
}

void FAnimationRecorderManager::HandleEndPIE(bool bSimulating)
{
	StopRecordingAllAnimations();

#if WITH_EDITOR
	FEditorDelegates::EndPIE.RemoveAll(this);
#endif
}

bool FAnimationRecorderManager::IsRecording(USkeletalMeshComponent* Component)
{
	for (FAnimRecorderInstance& Instance : RecorderInstances)
	{
		if (Instance.SkelComp == Component)
		{
			return Instance.Recorder->InRecording();
		}
	}

	return false;
}

bool FAnimationRecorderManager::IsRecording()
{
	for (FAnimRecorderInstance& Instance : RecorderInstances)
	{
		if (Instance.Recorder->InRecording())
		{
			return true;
		}
	}

	return false;
}

UAnimSequence* FAnimationRecorderManager::GetCurrentlyRecordingSequence(USkeletalMeshComponent* Component)
{
	for (FAnimRecorderInstance& Instance : RecorderInstances)
	{
		if (Instance.SkelComp == Component)
		{
			return Instance.Recorder->GetAnimationObject();
		}
	}

	return nullptr;
}

float FAnimationRecorderManager::GetCurrentRecordingTime(USkeletalMeshComponent* Component)
{
	for (FAnimRecorderInstance& Instance : RecorderInstances)
	{
		if (Instance.SkelComp == Component)
		{
			return Instance.Recorder->GetTimeRecorded();
		}
	}

	return 0.0f;
}

const FTransform&  FAnimationRecorderManager::GetInitialRootTransform(USkeletalMeshComponent* Component) const
{
	for (const FAnimRecorderInstance& Instance : RecorderInstances)
	{
		if (Instance.SkelComp == Component)
		{
			return Instance.Recorder->GetInitialRootTransform();
		}
	}
	return FTransform::Identity;
}

void FAnimationRecorderManager::StopRecordingAnimation(USkeletalMeshComponent* Component, bool bShowMessage)
{
	for (int32 Idx = 0; Idx < RecorderInstances.Num(); ++Idx)
	{
		FAnimRecorderInstance& Inst = RecorderInstances[Idx];
		if (Inst.SkelComp == Component)
		{
			// stop and finalize recoded data
			Inst.FinishRecording(bShowMessage);

			// remove instance, which will clean itself up
			RecorderInstances.RemoveAtSwap(Idx, EAllowShrinking::No);

			// all done
			break;
		}
	}
}

void FAnimationRecorderManager::StopRecordingDeadAnimations(bool bShowMessage)
{
	RecorderInstances.RemoveAll([&](FAnimRecorderInstance& Instance)
		{
			if (!Instance.SkelComp.IsValid())
			{
				// stop and finalize recorded data
				Instance.FinishRecording(bShowMessage);

				// make sure we are cleaned up
				return true;
			}

			return false;
		}
	);
}

void FAnimationRecorderManager::StopRecordingAllAnimations()
{
	for (auto& Inst : RecorderInstances)
	{
		Inst.FinishRecording();
	}

	RecorderInstances.Empty();
}

#undef LOCTEXT_NAMESPACE

