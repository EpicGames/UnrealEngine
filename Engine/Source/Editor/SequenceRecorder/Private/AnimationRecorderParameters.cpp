// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationRecorderParameters.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationRecorderParameters)

float UAnimationRecordingParameters::GetRecordingDurationSeconds()
{
	return bEndAfterDuration ? MaximumDurationSeconds : FAnimationRecordingSettings::UnboundedMaximumLength;
}

const FFrameRate& UAnimationRecordingParameters::GetRecordingFrameRate()
{
	return SampleFrameRate;
}
