// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimationRecordingSettings.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/FrameRate.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

#include "AnimationRecorderParameters.generated.h"

UCLASS(config = EditorPerProjectUserSettings)
class UAnimationRecordingParameters : public UObject
{
	GENERATED_BODY()

public:
	float GetRecordingDurationSeconds();
	const FFrameRate& GetRecordingFrameRate();

	virtual void PostInitProperties() override
	{
		Super::PostInitProperties();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (SampleRate > 0.f)
		{
			FString FrameRateFormat = FString::Printf(TEXT("1/%f"), SampleRate);
			TryParseString(SampleFrameRate, *FrameRateFormat);
			SampleRate = 0.f;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Sample frame-rate of the recorded animation  */
	UPROPERTY(EditAnywhere, config, Category = Options)
	FFrameRate SampleFrameRate = FAnimationRecordingSettings::DefaultSampleFrameRate;

	/** If enabled, this animation recording will automatically end after a set amount of time */
	UPROPERTY(EditAnywhere, config, Category = Options)
	bool bEndAfterDuration = false;

	/** The maximum duration of this animation recording */
	UPROPERTY(EditAnywhere, config, Category = Options, meta = (EditCondition = "bEndAfterDuration", UIMin = "0.03333", ClampMin = "0.03333"))
	float MaximumDurationSeconds = FAnimationRecordingSettings::DefaultMaximumLength;

	UE_DEPRECATED(5.0, "SampleRate has been deprecated and replaced with SampleFrameRate")
	UPROPERTY(config)
	float SampleRate = 0.f;
};
