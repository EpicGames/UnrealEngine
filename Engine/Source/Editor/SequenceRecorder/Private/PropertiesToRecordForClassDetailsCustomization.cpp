// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertiesToRecordForClassDetailsCustomization.h"

#include "DetailWidgetRow.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"
#include "SClassPropertyRecorderSettings.h"
#include "SequenceRecorderSettings.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

void FPropertiesToRecordForClassDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.
	NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];
}

void FPropertiesToRecordForClassDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedRef<IPropertyHandle> ClassProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPropertiesToRecordForClass, Class)).ToSharedRef();
	TSharedRef<IPropertyHandle> PropertiesProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPropertiesToRecordForClass, Properties)).ToSharedRef();

	ChildBuilder.AddProperty(ClassProperty);

	ChildBuilder.AddProperty(PropertiesProperty)
	.CustomWidget()
	.NameContent()
	[
		PropertiesProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(300.0f)
	.MaxDesiredWidth(400.0f)
	[
		SNew(SClassPropertyRecorderSettings, ClassProperty, PropertiesProperty, CustomizationUtils)
	];
}

void FPropertiesToRecordForActorClassDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.
	NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];
}

void FPropertiesToRecordForActorClassDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedRef<IPropertyHandle> ClassProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPropertiesToRecordForActorClass, Class)).ToSharedRef();
	TSharedRef<IPropertyHandle> PropertiesProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPropertiesToRecordForActorClass, Properties)).ToSharedRef();

	ChildBuilder.AddProperty(ClassProperty);

	ChildBuilder.AddProperty(PropertiesProperty)
	.CustomWidget()
	.NameContent()
	[
		PropertiesProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(300.0f)
	.MaxDesiredWidth(400.0f)
	[
		SNew(SClassPropertyRecorderSettings, ClassProperty, PropertiesProperty, CustomizationUtils)
	];
}
