// Copyright Epic Games, Inc. All Rights Reserved.

#include "SClassPropertyRecorderSettings.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "DetailsViewArgs.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Sections/MovieSceneMultiPropertyRecorder.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/SubclassOf.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

class IDetailLayoutBuilder;
class UObject;

#define LOCTEXT_NAMESPACE "SClassPropertyRecorderSettings"

void SClassPropertyRecorderSettings::Construct(const FArguments& Args, const TSharedRef<class IPropertyHandle>& InClassHandle, const TSharedRef<class IPropertyHandle>& InPropertiesHandle, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertiesHandle = InPropertiesHandle;
	ClassHandle = InClassHandle;

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(this, &SClassPropertyRecorderSettings::GetText)
			.Font(CustomizationUtils.GetRegularFont())
			.AutoWrapText(true)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("ChoosePropertiesButtonToolTip", "Choose properties to be recorded for this class"))
			.OnClicked(this, &SClassPropertyRecorderSettings::HandleChoosePropertiesButtonClicked)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
			]
		]
	];
}

FText SClassPropertyRecorderSettings::GetText() const
{
	if (PropertiesHandle.IsValid())
	{
		TArray<FName>* PropertiesToRecord = GetPropertyNameArray();
		if (PropertiesToRecord != nullptr && PropertiesToRecord->Num() > 0)
		{
			FText Text = FText::Format(LOCTEXT("SinglePropertyToRecordFormat", "{0}"), FText::FromName((*PropertiesToRecord)[0]));
			for (int32 PropertyIndex = 1; PropertyIndex < PropertiesToRecord->Num(); ++PropertyIndex)
			{
				Text = FText::Format(LOCTEXT("PropertiesToRecordFormat", "{0}, {1}"), Text, FText::FromName((*PropertiesToRecord)[PropertyIndex]));
			}
			return Text;
		}
	}

	return LOCTEXT("NoPropertiesToRecord", "None");
}

FReply SClassPropertyRecorderSettings::HandleChoosePropertiesButtonClicked()
{
	if (ClassHandle.IsValid() && PropertiesHandle.IsValid())
	{
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bHideSelectionTip = true;
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

		DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SClassPropertyRecorderSettings::ShouldShowProperty));
		DetailsView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SClassPropertyRecorderSettings::IsPropertyReadOnly));
		DetailsView->SetExtensionHandler(SharedThis(this));
		DetailsView->SetDisableCustomDetailLayouts(true);

		TArray<void*> RawData;
		ClassHandle->AccessRawData(RawData);
		check(RawData.Num() > 0);
		TSubclassOf<UObject>& Class = *reinterpret_cast<TSubclassOf<UObject>*>(RawData[0]);
		DetailsView->SetObject(Class.GetDefaultObject());

		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(LOCTEXT("PropertyPickerWindowTitle", "Choose Properties to Be Recorded"))
			.ClientSize(FVector2D(400, 550));

		Window->SetContent(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("PropertyWindow.WindowBorder")))
			[
				DetailsView
			]
		);

		// Parent the window to the root window
		FSlateApplication::Get().AddModalWindow(Window, FGlobalTabmanager::Get()->GetRootWindow().ToSharedRef());
	}
	return FReply::Handled();
}

bool ShouldShowProperty_Recursive(const FProperty* InProperty)
{
	if (InProperty->IsA<FStructProperty>())
	{
		const FStructProperty* StructProperty = CastFieldChecked<const FStructProperty>(InProperty);
		for (const FProperty* Property = StructProperty->Struct->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			if (Property->HasAnyPropertyFlags(CPF_Interp) && FMovieSceneMultiPropertyRecorder::CanPropertyBeRecorded(*Property))
			{
				return true;
			}
			else if(ShouldShowProperty_Recursive(Property))
			{
				return true;
			}
		}
	}

	return false;
}

bool SClassPropertyRecorderSettings::ShouldShowProperty(const FPropertyAndParent& InPropertyAndParent) const
{
	bool bShow = InPropertyAndParent.Property.HasAnyPropertyFlags(CPF_Interp) && FMovieSceneMultiPropertyRecorder::CanPropertyBeRecorded(InPropertyAndParent.Property);

	// we also need to recurse into sub-properties to see if we should show them too
	return bShow || ShouldShowProperty_Recursive(&InPropertyAndParent.Property);
}

bool SClassPropertyRecorderSettings::IsPropertyReadOnly(const FPropertyAndParent& InPropertyAndParent) const
{
	return true;
}

bool SClassPropertyRecorderSettings::IsPropertyExtendable(const UClass* InObjectClass, const class IPropertyHandle& PropertyHandle) const
{
	return PropertyHandle.GetProperty()->HasAnyPropertyFlags(CPF_Interp) && FMovieSceneMultiPropertyRecorder::CanPropertyBeRecorded(*PropertyHandle.GetProperty());
}

void SClassPropertyRecorderSettings::ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailLayoutBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	ECheckBoxState InitialState = ECheckBoxState::Unchecked;
	FName PropertyName = *PropertyHandle->GeneratePathToProperty();

	TArray<FName>* PropertiesToRecord = GetPropertyNameArray();
	if (PropertiesToRecord != nullptr)
	{
		InitialState = PropertiesToRecord->Contains(PropertyName) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	InWidgetRow.ExtensionContent()
	[
		SNew(SCheckBox)
			.OnCheckStateChanged(this, &SClassPropertyRecorderSettings::HandlePropertyCheckStateChanged, PropertyHandle)
			.IsChecked(InitialState)
	];
}

void SClassPropertyRecorderSettings::HandlePropertyCheckStateChanged(ECheckBoxState InState, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	if (PropertyHandle.IsValid() && PropertiesHandle.IsValid())
	{
		TArray<FName>* PropertiesToRecord = GetPropertyNameArray();

		if (PropertiesToRecord != nullptr)
		{
			FName PropertyPath = *PropertyHandle->GeneratePathToProperty();
			if (InState == ECheckBoxState::Checked)
			{
				PropertiesToRecord->AddUnique(PropertyPath);
			}
			else
			{
				PropertiesToRecord->Remove(PropertyPath);
			}

			PropertiesHandle->NotifyPostChange(InState == ECheckBoxState::Checked ? EPropertyChangeType::ArrayAdd : EPropertyChangeType::ArrayRemove);
		}
	}
}

TArray<FName>* SClassPropertyRecorderSettings::GetPropertyNameArray() const
{
	check(PropertiesHandle.IsValid());

	TArray<void*> RawData;
	PropertiesHandle->AccessRawData(RawData);
	check(RawData.Num() > 0);
	if (RawData[0] != nullptr)
	{
		return reinterpret_cast<TArray<FName>*>(RawData[0]);
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
