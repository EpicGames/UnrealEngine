// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IDetailPropertyExtensionHandler.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FDetailWidgetRow;
class FName;
class IDetailLayoutBuilder;
class IPropertyHandle;
class IPropertyTypeCustomizationUtils;
class UClass;
struct FPropertyAndParent;

class SClassPropertyRecorderSettings : public SCompoundWidget, public IDetailPropertyExtensionHandler
{
	SLATE_BEGIN_ARGS(SClassPropertyRecorderSettings)
	{
	}

	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<class IPropertyHandle>& InClassHandle, const TSharedRef<class IPropertyHandle>& InPropertiesHandle, IPropertyTypeCustomizationUtils& CustomizationUtils);

private:
	/** Get the text to display describing the properties we are recording */
	FText GetText() const;

	/** Handle clicking the 'edit' button */
	FReply HandleChoosePropertiesButtonClicked();

	/** Hide non-keyable properties in the picker window */
	bool ShouldShowProperty(const FPropertyAndParent& InPropertyAndParent) const;

	/** Make all properties read-only */
	bool IsPropertyReadOnly(const FPropertyAndParent& InPropertyAndParent) const;

	/** Handle a property being marked for recording */
	void HandlePropertyCheckStateChanged(ECheckBoxState InState, TSharedPtr<IPropertyHandle> PropertyHandle);

	/** Helper function to get the actual name array from the property handle */
	TArray<FName>* GetPropertyNameArray() const;

	/** IDetailPropertyExtensionHandler interface */
	virtual bool IsPropertyExtendable(const UClass* InObjectClass, const class IPropertyHandle& PropertyHandle) const override;
	virtual void ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailLayoutBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle) override;

private:
	/** Handle to the property we are editing */
	TSharedPtr<class IPropertyHandle> PropertiesHandle;

	/** Handle to the class property in the struct */
	TSharedPtr<class IPropertyHandle> ClassHandle;
};
