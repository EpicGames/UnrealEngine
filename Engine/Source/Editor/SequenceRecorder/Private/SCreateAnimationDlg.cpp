// Copyright Epic Games, Inc. All Rights Reserved.


#include "SCreateAnimationDlg.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "PropertyEditorModule.h"
#include "Styling/CoreStyle.h"
//#include "Persona.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#define LOCTEXT_NAMESPACE "SCreateAnimationDlg"

/////////////////////////////////////////////////////
/////////////////////////////////////////////////////
/////////////////////////////////////////////////////
/////////////////////////////////////////////////////

/////////////////////////////////////////////////////
// Create Animation dialog for recording animation
/////////////////////////////////////////////////////

FText SCreateAnimationDlg::LastUsedAssetPath;

void SCreateAnimationDlg::Construct(const FArguments& InArgs)
{
	AssetPath = FText::FromString(FPackageName::GetLongPackagePath(InArgs._DefaultAssetPath.ToString()));
	AssetName = FText::FromString(FPackageName::GetLongPackageAssetName(InArgs._DefaultAssetPath.ToString()));

	if (AssetPath.IsEmpty())
	{
		AssetPath = LastUsedAssetPath;
		// still empty?
		if (AssetPath.IsEmpty())
		{
			AssetPath = FText::FromString(TEXT("/Game"));
		}
	}
	else
	{
		LastUsedAssetPath = AssetPath;
	}

	if (AssetName.IsEmpty())
	{
		// find default name for them
		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		FString OutPackageName, OutAssetName;
		FString PackageName = AssetPath.ToString() + TEXT("/NewAnimation");

		AssetToolsModule.Get().CreateUniqueAssetName(PackageName,  TEXT(""), OutPackageName, OutAssetName);
		AssetName = FText::FromString(OutAssetName);
	}

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = AssetPath.ToString();
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SCreateAnimationDlg::OnPathChange);
	PathPickerConfig.bAddDefaultPath = true;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.ViewIdentifier = "AnimationRecorder";

	TSharedPtr<class IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	GetMutableDefault<UAnimationRecordingParameters>()->LoadConfig();
	DetailsView->SetObject(GetMutableDefault<UAnimationRecordingParameters>(), true);
	DetailsView->OnFinishedChangingProperties().AddLambda([](const FPropertyChangedEvent& InEvent) { GetMutableDefault<UAnimationRecordingParameters>()->SaveConfig(); });

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("SCreateAnimationDlg_Title", "Create New Animation Object"))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		//.SizingRule( ESizingRule::Autosized )
		.ClientSize(FVector2D(450,450))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot() // Add user input block
			.Padding(2)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SelectPath", "Select Path to create animation"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
					]

					+SVerticalBox::Slot()
					.FillHeight(1)
					.Padding(3)
					[
						ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SSeparator)
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(3)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0, 0, 10, 0)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("AnimationName", "Animation Name"))
						]

						+SHorizontalBox::Slot()
						[
							SNew(SEditableTextBox)
							.Text(AssetName)
							.OnTextCommitted(this, &SCreateAnimationDlg::OnNameChange)
							.MinDesiredWidth(250)
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					[
						DetailsView.ToSharedRef()
					]

				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0,0)
				[
					SNew(SButton) 
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("OK", "OK"))
					.OnClicked(this, &SCreateAnimationDlg::OnButtonClick, EAppReturnType::Ok)
				]
				+SUniformGridPanel::Slot(1,0)
					[
						SNew(SButton) 
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.Text(LOCTEXT("Cancel", "Cancel"))
						.OnClicked(this, &SCreateAnimationDlg::OnButtonClick, EAppReturnType::Cancel)
					]
			]
		]);
}

void SCreateAnimationDlg::OnNameChange(const FText& NewName, ETextCommit::Type CommitInfo)
{
	AssetName = NewName;
}

void SCreateAnimationDlg::OnPathChange(const FString& NewPath)
{
	AssetPath = FText::FromString(NewPath);
	LastUsedAssetPath = AssetPath;
}

FReply SCreateAnimationDlg::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;

	if (ButtonID != EAppReturnType::Cancel)
	{
		if (!ValidatePackage())
		{
			// reject the request
			return FReply::Handled();
		}
	}

	RequestDestroyWindow();

	return FReply::Handled();
}

/** Ensures supplied package name information is valid */
bool SCreateAnimationDlg::ValidatePackage()
{
	FText Reason;
	FString FullPath = GetFullAssetPath();

	if(!FPackageName::IsValidLongPackageName(FullPath, false, &Reason)
		|| !FName(*AssetName.ToString()).IsValidObjectName(Reason))
	{
		FMessageDialog::Open(EAppMsgType::Ok, Reason);
		return false;
	}

	return true;
}

EAppReturnType::Type SCreateAnimationDlg::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

FString SCreateAnimationDlg::GetAssetPath()
{
	return AssetPath.ToString();
}

FString SCreateAnimationDlg::GetAssetName()
{
	return AssetName.ToString();
}

FString SCreateAnimationDlg::GetFullAssetPath()
{
	return AssetPath.ToString() + "/" + AssetName.ToString();
}

#undef LOCTEXT_NAMESPACE
