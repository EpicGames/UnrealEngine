// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "IDetailCustomization.h"
#include "AnimationRecorderParameters.h"

class SCreateAnimationDlg : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SCreateAnimationDlg)
	{}
	
	SLATE_ARGUMENT(FText, DefaultAssetPath)
	SLATE_END_ARGS()

	SCreateAnimationDlg()
		: UserResponse(EAppReturnType::Cancel)
	{}

	void Construct(const FArguments& InArgs);

public:
	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

	/** Gets the resulting asset path */
	FString GetAssetPath();

	/** Gets the resulting asset name */
	FString GetAssetName();

	/** Gets the resulting full asset path (path+'/'+name) */
	FString GetFullAssetPath();

	/** Gets the resulting sample rate and recording duration */
	UAnimationRecordingParameters* GetRecordingParameters() const { return GetMutableDefault<UAnimationRecordingParameters>(); }


protected:
	void OnPathChange(const FString& NewPath);
	void OnNameChange(const FText& NewName, ETextCommit::Type CommitInfo);
	FReply OnButtonClick(EAppReturnType::Type ButtonID);

	bool ValidatePackage();

	EAppReturnType::Type UserResponse;
	FText AssetPath;
	FText AssetName;

	static FText LastUsedAssetPath;
};
