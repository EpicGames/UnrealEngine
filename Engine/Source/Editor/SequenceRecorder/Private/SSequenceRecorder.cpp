// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSequenceRecorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SOverlay.h"
#include "ActorRecording.h"
#include "Modules/ModuleManager.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Styling/AppStyle.h"
#include "AnimationRecorder.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "SequenceRecorderCommands.h"
#include "SequenceRecorderSettings.h"
#include "SequenceRecorderUtils.h"
#include "SequenceRecorder.h"
#include "SDropTarget.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "SequenceRecorderActorGroup.h"
#include "ISequenceRecorderExtender.h"
#include "ISinglePropertyView.h"
#include "ActorGroupDetailsCustomization.h"
#include "EditorSupportDelegates.h"
#include "Editor.h"
#include "LevelEditor.h"
#include "Templates/UnrealTemplate.h"
#include "EngineAnalytics.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "IDocumentation.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Misc/ConfigCacheIni.h"


#define LOCTEXT_NAMESPACE "SequenceRecorder"

static const FName ActiveColumnName(TEXT("Active"));
static const FName ItemColumnName(TEXT("Item"));
static const FName TargetNameColumnName(TEXT("Name"));
static const FName AnimationColumnName(TEXT("Animation"));
static const FName TakeColumnName(TEXT("Take"));
static const FName FrameColumnName(TEXT("Frame"));

/** A widget to display information about an animation recording in the list view */
class SSequenceRecorderActorListRow : public SMultiColumnTableRow<UActorRecording*>
{
public:
	SLATE_BEGIN_ARGS(SSequenceRecorderActorListRow) {}

	/** The list item for this row */
	SLATE_ARGUMENT(UActorRecording*, Recording)

	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		RecordingPtr = Args._Recording;

		SMultiColumnTableRow<UActorRecording*>::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			OwnerTableView
			);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == ActiveColumnName)
		{
			return
				SNew(SButton)
				.ContentPadding(0)
				.OnClicked(this, &SSequenceRecorderActorListRow::ToggleRecordingActive)
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.ToolTipText(LOCTEXT("ActiveButtonToolTip", "Toggle Recording Active"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Content()
				[
					SNew(SImage)
					.Image(this, &SSequenceRecorderActorListRow::GetActiveBrushForRecording)
				];
		}
		else if (ColumnName == ItemColumnName)
		{
			return SNew( SHorizontalBox )
				+SHorizontalBox::Slot()
				.Padding( 2.0f, 0.0f, 2.0f, 0.0f )
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SSequenceRecorderActorListRow::GetRecordingActorName)))
				];
		}
		else if (ColumnName == TargetNameColumnName)
		{
			return SNew( SHorizontalBox )
				+SHorizontalBox::Slot()
				.Padding( 2.0f, 0.0f, 2.0f, 0.0f )
				.VAlign(VAlign_Center)
				[
					SNew(SEditableTextBox)
					.ToolTipText(LOCTEXT("TargetNameToolTip", "Optional target track name to record to"))
					.Text(this, &SSequenceRecorderActorListRow::GetRecordingTargetName)
					.OnTextChanged(this, &SSequenceRecorderActorListRow::SetRecordingTargetName)
				];
		}
		else if (ColumnName == AnimationColumnName)
		{
			return SNew( SHorizontalBox )
				+SHorizontalBox::Slot()
				.Padding( 2.0f, 0.0f, 2.0f, 0.0f )
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SSequenceRecorderActorListRow::GetRecordingAnimationName)))
				];
		}
		else if (ColumnName == TakeColumnName)
		{
			return
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
				.BorderBackgroundColor(this, &SSequenceRecorderActorListRow::GetRecordingTakeBorderColor)
				.VAlign(VAlign_Center)
				[
					SNew(SNumericEntryBox<int32>)
					.Value(this, &SSequenceRecorderActorListRow::GetRecordingTake)
					.OnValueChanged(this, &SSequenceRecorderActorListRow::SetRecordingTake)
				];
		}

		return SNullWidget::NullWidget;
	}

private:

	FReply ToggleRecordingActive()
	{
		if (RecordingPtr.IsValid())
		{
			 RecordingPtr.Get()->bActive = !RecordingPtr.Get()->bActive;
		}
		return FReply::Handled();
	}

	const FSlateBrush* GetActiveBrushForRecording() const
	{
		if (RecordingPtr.IsValid() && RecordingPtr.Get()->bActive)
		{
			return FAppStyle::GetBrush("SequenceRecorder.Common.RecordingActive");
		}
		else
		{
			return FAppStyle::GetBrush("SequenceRecorder.Common.RecordingInactive");
		}
	}

	FText GetRecordingActorName() const
	{
		FText ActorName(LOCTEXT("InvalidActorName", "None"));
		if (RecordingPtr.IsValid() && RecordingPtr.Get()->GetActorToRecord() != nullptr)
		{
			ActorName = FText::FromString(RecordingPtr.Get()->GetActorToRecord()->GetActorLabel());
		}

		return ActorName;
	}

	FText GetRecordingTargetName() const
	{
		FText TargetName(LOCTEXT("InvalidActorName", "None"));
		if (RecordingPtr.IsValid())
		{
			if (RecordingPtr.Get()->TargetName.IsEmpty())
			{
				if (RecordingPtr.Get()->GetActorToRecord() != nullptr)
				{
					TargetName = FText::FromString(RecordingPtr.Get()->GetActorToRecord()->GetActorLabel());
				}
			}
			else
			{
				TargetName = RecordingPtr.Get()->TargetName;
			}
		}

		return TargetName;
	}

	void SetRecordingTargetName(const FText& InText)
	{
		if (RecordingPtr.IsValid())
		{
			RecordingPtr.Get()->TargetName = InText;

			// Reset take number and target level sequence
			RecordingPtr.Get()->TakeNumber = 1;
			RecordingPtr.Get()->TargetLevelSequence = nullptr;
		}
	}

	FText GetRecordingAnimationName() const
	{
		FText AnimationName(LOCTEXT("InvalidAnimationName", "None"));
		if (RecordingPtr.IsValid())
		{
			if(!RecordingPtr.Get()->bSpecifyTargetAnimation)
			{
				AnimationName = LOCTEXT("AutoCreatedAnimationName", "Auto");
			}
			else if(RecordingPtr.Get()->TargetAnimation != nullptr)
			{
				AnimationName = FText::FromString(RecordingPtr.Get()->TargetAnimation->GetName());
			}
		}

		return AnimationName;
	}

	TOptional<int32> GetRecordingTake() const
	{
		if (RecordingPtr.IsValid())
		{
			return RecordingPtr.Get()->TakeNumber;
		}

		return 1;
	}
	
	FSlateColor GetRecordingTakeBorderColor() const
	{
		if (RecordingPtr.IsValid())
		{
			const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

			const FString SequenceName = FSequenceRecorder::Get().GetSequenceRecordingName();
			TOptional<int32> TakeNumber = GetRecordingTake();
			FString TargetName = GetRecordingTargetName().ToString();
			FString SessionName = !SequenceName.IsEmpty() ? SequenceName : TEXT("RecordedSequence");
			FString AssetPath = FSequenceRecorder::Get().GetSequenceRecordingBasePath() / SessionName / TargetName;

			FString TakeName = SequenceRecorderUtils::MakeTakeName(TargetName, SessionName, TakeNumber.GetValue());

			if (SequenceRecorderUtils::DoesTakeExist(AssetPath, TargetName, SessionName, TakeNumber.GetValue()))
			{
				return FLinearColor::Red;
			}
		}

		return FLinearColor::White;
	}

	void SetRecordingTake(int32 InTakeNumber)
	{
		if (RecordingPtr.IsValid())
		{
			RecordingPtr.Get()->TakeNumber = InTakeNumber;
		}
	}

	TWeakObjectPtr<UActorRecording> RecordingPtr;
};

static void OnDocLinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* Url = Metadata.Find(TEXT("href"));
	if (Url)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("DocLink"), *Url));

			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.SequenceRecorder.DeprecationWarning.DocLinkClicked"), EventAttributes);
		}

		IDocumentation::Get()->Open(*Url, FDocumentationSourceInfo(TEXT("editor")));
	}
}

class SSequenceRecorderDeprecationMessage : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSequenceRecorderDeprecationMessage)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor(0.4f, 0.f, 0.f, 1.f))
			.BorderImage(FAppStyle::GetBrush(TEXT("WhiteBrush")))
			.Visibility(this, &SSequenceRecorderDeprecationMessage::GetWarningMessageVisibility)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(4, 0, 0, 0)
				[
					SNew(SRichTextBlock)
					.Text(LOCTEXT("SequenceRecorderLastVersionSupported", "As of 4.26, Sequence Recorder is no longer supported by UE4 and will be removed from the engine in the near future. Once removed, you will <NormalText.Important>no longer be able to run Sequence Recorder</>.\nPlease use <a id=\"TakeRecorderDoc\" href=\"/Engine/Sequencer/Workflow/TakeRecorder\" style=\"DarkHyperlink\">Take Recorder</>"))
					.AutoWrapText(true)
					.DecoratorStyleSet(&FAppStyle::Get())
					+ SRichTextBlock::HyperlinkDecorator(TEXT("TakeRecorderDoc"), FSlateHyperlinkRun::FOnClick::CreateStatic(&OnDocLinkClicked))

				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("DismissSequenceRecorderSupportWarning", "Dismiss"))
					.OnClicked(this, &SSequenceRecorderDeprecationMessage::DismissWarningForever)
				]
			]
		];
	}

	FReply DismissWarningForever() 
	{
		if (GConfig)
		{
			GConfig->SetBool(TEXT("SequenceRecorder"), TEXT("HasDismissedDeprecationWarning"), true, GEditorIni);

			if (FEngineAnalytics::IsAvailable())
			{
				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.SequenceRecorder.DeprecationWarning.Dimissed"));
			}
		}

		return FReply::Handled();
	}

	EVisibility GetWarningMessageVisibility() const
	{
		if (GConfig)
		{
			bool bDismissed = false;
			GConfig->GetBool(TEXT("SequenceRecorder"), TEXT("HasDismissedDeprecationWarning"), bDismissed, GEditorIni);

			return bDismissed ? EVisibility::Collapsed : EVisibility::Visible;
		}

		return EVisibility::Collapsed;
	}
};

void SSequenceRecorder::Construct(const FArguments& Args)
{
	bInsideSelectionChanged = false;
	CommandList = MakeShareable(new FUICommandList);

	BindCommands();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;

	SelectedRecordingItemDetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
	SequenceRecordingDetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );
	RecordingGroupDetailsView = PropertyEditorModule.CreateDetailView( DetailsViewArgs );

	TWeakPtr<SSequenceRecorder> WeakPtr = SharedThis(this);
	RecordingGroupDetailsView->RegisterInstancedCustomPropertyLayout(USequenceRecorderActorGroup::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FActorGroupDetailsCustomization::MakeInstance, WeakPtr));

	SequenceRecordingDetailsView->SetObject(GetMutableDefault<USequenceRecorderSettings>());
	RecordingGroupDetailsView->SetObject(GetMutableDefault<USequenceRecorderActorGroup>());

	FToolBarBuilder ToolBarBuilder(CommandList, FMultiBoxCustomization::None);

	ToolBarBuilder.BeginSection(TEXT("Recording"));
	{
		ToolBarBuilder.AddToolBarButton(FSequenceRecorderCommands::Get().RecordAll);
		ToolBarBuilder.AddToolBarButton(FSequenceRecorderCommands::Get().StopAll);
	}
	ToolBarBuilder.EndSection();

	ToolBarBuilder.BeginSection(TEXT("RecordingManagement"));
	{
		ToolBarBuilder.AddToolBarButton(FSequenceRecorderCommands::Get().AddRecording);
		ToolBarBuilder.AddToolBarButton(FSequenceRecorderCommands::Get().AddCurrentPlayerRecording);
		ToolBarBuilder.AddToolBarButton(FSequenceRecorderCommands::Get().RemoveRecording);
		ToolBarBuilder.AddToolBarButton(FSequenceRecorderCommands::Get().RemoveAllRecordings);
	}
	ToolBarBuilder.EndSection();

	TSharedPtr<SVerticalBox> DetailsViewVerticalBox;
	TSharedPtr<SVerticalBox> ListViewVerticalBox;

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Vertical)
		+SSplitter::Slot()
		.Value(0.33f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSequenceRecorderDeprecationMessage)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 4.0f, 0.0f, 0.0f))
			[
				ToolBarBuilder.MakeWidget()
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(FMargin(0.0f, 4.0f, 0.0f, 0.0f))
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(4.0f, 4.0f))
				[
					SNew(SOverlay)
					+SOverlay::Slot()
					[
						SAssignNew(ListViewVerticalBox, SVerticalBox)
						+SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							SNew( SDropTarget )
							.OnAllowDrop( this, &SSequenceRecorder::OnRecordingActorListAllowDrop )
							.OnDropped( this, &SSequenceRecorder::OnRecordingActorListDrop )
							.Content()
							[
								SAssignNew(ActorListView, SListView<UActorRecording*>)
								.ListItemsSource(&FSequenceRecorder::Get().GetQueuedActorRecordings())
								.SelectionMode(ESelectionMode::SingleToggle)
								.OnGenerateRow(this, &SSequenceRecorder::MakeListViewWidget)
								.OnSelectionChanged(this, &SSequenceRecorder::OnActorListSelectionChanged)
								.HeaderRow
								(
									SNew(SHeaderRow)
									+ SHeaderRow::Column(ActiveColumnName)
									.FillWidth(10.0f)
									.DefaultLabel(LOCTEXT("ActiveColumnName", "Active"))

									+ SHeaderRow::Column(ItemColumnName)
									.FillWidth(30.0f)
									.DefaultLabel(LOCTEXT("ActorHeaderName", "Actor"))

									+ SHeaderRow::Column(TargetNameColumnName)
									.FillWidth(30.0f)
									.DefaultLabel(LOCTEXT("TargetNameHeaderName", "Name"))

									+ SHeaderRow::Column(AnimationColumnName)
									.FillWidth(20.0f)
									.DefaultLabel(LOCTEXT("AnimationHeaderName", "Anim"))

									+ SHeaderRow::Column(TakeColumnName)
									.FillWidth(10.0f)
									.DefaultLabel(LOCTEXT("TakeHeaderName", "Take"))
								)
							]
						]
					]
					+SOverlay::Slot()
					[
						SNew(SVerticalBox)	
						+SVerticalBox::Slot()
						.VAlign(VAlign_Bottom)
						.MaxHeight(2.0f)
						[
							SAssignNew(DelayProgressBar, SProgressBar)
							.Percent(this, &SSequenceRecorder::GetDelayPercent)
							.Visibility(this, &SSequenceRecorder::GetDelayProgressVisibilty)
						]
					]
				]
			]
		]
		+SSplitter::Slot()
		.Value(0.66f)
		[
			SNew(SScrollBox)
			+SScrollBox::Slot()
			[
				SAssignNew(DetailsViewVerticalBox, SVerticalBox)
				.IsEnabled_Lambda([]() { return !FSequenceRecorder::Get().IsRecording(); })
				+SVerticalBox::Slot()
				[
					RecordingGroupDetailsView.ToSharedRef()
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SequenceRecordingDetailsView.ToSharedRef()
				]
			]
		]
	];

	// Add additional Detail view
	for (TSharedPtr<ISequenceRecorderExtender> Extender : FSequenceRecorder::Get().GetSequenceRecorderExtenders())
	{
		TSharedPtr<IDetailsView> DetailView = Extender->MakeSettingDetailsView();
		if (DetailView.IsValid())
		{
			DetailsViewVerticalBox->AddSlot()
			.AutoHeight()
			[
				DetailView.ToSharedRef()
			];
		}
	}
	DetailsViewVerticalBox->AddSlot()
	.AutoHeight()
	[
		SelectedRecordingItemDetailsView.ToSharedRef()
	];

	// Add additional ListView
	{
		for (TSharedPtr<ISequenceRecorderExtender> Extender : FSequenceRecorder::Get().GetSequenceRecorderExtenders())
		{
			TSharedPtr<SListView<USequenceRecordingBase*>> CreatedListView;
			TSharedPtr<SWidget> ListWidget = Extender->MakeListWidget(CreatedListView, ISequenceRecorderExtender::FListViewSelectionChanged::CreateRaw(this, &SSequenceRecorder::OnListSelectionChanged));
			if (ListWidget.IsValid() && CreatedListView.IsValid())
			{
				ListViewVerticalBox->AddSlot()
				.FillHeight(1.0f)
				[
					ListWidget.ToSharedRef()
				];

				ExtenderListViews.Add(CreatedListView);
			}
		}
	}
	ListViewVerticalBox->AddSlot()
	.AutoHeight()
	[
		SNew(STextBlock)
		.Text(this, &SSequenceRecorder::GetTargetSequenceName)
	];

	// Register refresh timer
	if (!ActiveTimerHandle.IsValid())
	{
		ActiveTimerHandle = RegisterActiveTimer(0.1f, FWidgetActiveTimerDelegate::CreateSP(this, &SSequenceRecorder::HandleRefreshItems));
	}

	FSequenceRecorder::Get().OnRecordingGroupAddedDelegate.AddRaw(this, &SSequenceRecorder::HandleRecordingGroupAddedToSequenceRecorder);
#if WITH_EDITOR
	FEditorSupportDelegates::PrepareToCleanseEditorObject.AddRaw(this, &SSequenceRecorder::HandleMapUnload);
#endif
}

SSequenceRecorder::~SSequenceRecorder()
{
#if WITH_EDITOR
	FEditorSupportDelegates::PrepareToCleanseEditorObject.RemoveAll(this);
#endif
	FSequenceRecorder::Get().OnRecordingGroupAddedDelegate.RemoveAll(this);

}

void SSequenceRecorder::HandleMapUnload(UObject* Object)
{
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	if (EditorWorld == Object)
	{
		// When a map object is about to be GC'd we want to make sure the UI releases all references
		// to anything that is owned by the scene.
		SelectedRecordingItemDetailsView->SetObject(nullptr);

		// Force the list view to rebuild after clearing it's data source. This clears the list view's
		// widget children. The ListView should only contain TWeakObjectPtrs but it's holding a hard
		// reference anyways that's causes a gc leak on map change.
		FSequenceRecorder::Get().ClearQueuedRecordings();
		ActorListView->RebuildList();
		for (TSharedPtr<SListView<USequenceRecordingBase*>>& ListView : ExtenderListViews)
		{
			ListView->RebuildList();
		}

		// We also want to construct a new mutable default so it resets the recording paths to the
		// default paths for the new map.
		RecordingGroupDetailsView->SetObject(GetMutableDefault<USequenceRecorderActorGroup>());
	}
}

void SSequenceRecorder::BindCommands()
{
	CommandList->MapAction(FSequenceRecorderCommands::Get().RecordAll,
		FExecuteAction::CreateSP(this, &SSequenceRecorder::HandleRecord),
		FCanExecuteAction::CreateSP(this, &SSequenceRecorder::CanRecord),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SSequenceRecorder::IsRecordVisible)
		);

	CommandList->MapAction(FSequenceRecorderCommands::Get().StopAll,
		FExecuteAction::CreateSP(this, &SSequenceRecorder::HandleStopAll),
		FCanExecuteAction::CreateSP(this, &SSequenceRecorder::CanStopAll),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SSequenceRecorder::IsStopAllVisible)
		);

	CommandList->MapAction(FSequenceRecorderCommands::Get().AddRecording,
		FExecuteAction::CreateSP(this, &SSequenceRecorder::HandleAddRecording),
		FCanExecuteAction::CreateSP(this, &SSequenceRecorder::CanAddRecording)
		);

	CommandList->MapAction(FSequenceRecorderCommands::Get().AddCurrentPlayerRecording,
		FExecuteAction::CreateSP(this, &SSequenceRecorder::HandleAddCurrentPlayerRecording),
		FCanExecuteAction::CreateSP(this, &SSequenceRecorder::CanAddCurrentPlayerRecording)
		);

	CommandList->MapAction(FSequenceRecorderCommands::Get().RemoveRecording,
		FExecuteAction::CreateSP(this, &SSequenceRecorder::HandleRemoveRecording),
		FCanExecuteAction::CreateSP(this, &SSequenceRecorder::CanRemoveRecording)
		);

	CommandList->MapAction(FSequenceRecorderCommands::Get().RemoveAllRecordings,
		FExecuteAction::CreateSP(this, &SSequenceRecorder::HandleRemoveAllRecordings),
		FCanExecuteAction::CreateSP(this, &SSequenceRecorder::CanRemoveAllRecordings)
		);

	CommandList->MapAction(FSequenceRecorderCommands::Get().AddRecordingGroup,
		FExecuteAction::CreateSP(this, &SSequenceRecorder::HandleAddRecordingGroup),
		FCanExecuteAction::CreateSP(this, &SSequenceRecorder::CanAddRecordingGroup)
	);

	CommandList->MapAction(FSequenceRecorderCommands::Get().RemoveRecordingGroup,
		FExecuteAction::CreateSP(this, &SSequenceRecorder::HandleRemoveRecordingGroup),
		FCanExecuteAction::CreateSP(this, &SSequenceRecorder::CanRemoveRecordingGroup)
	);

	CommandList->MapAction(FSequenceRecorderCommands::Get().DuplicateRecordingGroup,
		FExecuteAction::CreateSP(this, &SSequenceRecorder::HandleDuplicateRecordingGroup),
		FCanExecuteAction::CreateSP(this, &SSequenceRecorder::CanDuplicateRecordingGroup)
	);

	// Append to level editor module so that shortcuts are accessible in level editor
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	LevelEditorModule.GetGlobalLevelEditorActions()->Append(CommandList.ToSharedRef());
}

TSharedRef<ITableRow> SSequenceRecorder::MakeListViewWidget(UActorRecording* Recording, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return
		SNew(SSequenceRecorderActorListRow, OwnerTable)
		.Recording(Recording);
}

void SSequenceRecorder::OnActorListSelectionChanged(UActorRecording* Recording, ESelectInfo::Type SelectionType)
{
	if (bInsideSelectionChanged)
	{
		return;
	}

	TGuardValue<bool> TmpValue(bInsideSelectionChanged, true);
	for (TSharedPtr<ISequenceRecorderExtender> Extender : FSequenceRecorder::Get().GetSequenceRecorderExtenders())
	{
		Extender->SetListViewSelection(nullptr);
	}

	if(Recording)
	{
		SelectedRecordingItemDetailsView->SetObject(Recording);
	}
	else
	{
		SelectedRecordingItemDetailsView->SetObject(nullptr);
	}
}

void SSequenceRecorder::OnListSelectionChanged(USequenceRecordingBase* InRecoderBase)
{
	if (bInsideSelectionChanged)
	{
		return;
	}

	TGuardValue<bool> TmpValue(bInsideSelectionChanged, true);
	ActorListView->SetSelection(nullptr, ESelectInfo::Direct);
	for (TSharedPtr<ISequenceRecorderExtender> Extender : FSequenceRecorder::Get().GetSequenceRecorderExtenders())
	{
		Extender->SetListViewSelection(nullptr);
	}

	if (InRecoderBase)
	{
		SelectedRecordingItemDetailsView->SetObject(InRecoderBase);
	}
	else
	{
		SelectedRecordingItemDetailsView->SetObject(nullptr);
	}
}

void SSequenceRecorder::HandleRecord()
{
	FSequenceRecorder::Get().StartRecording();
}

bool SSequenceRecorder::CanRecord() const
{
	return FSequenceRecorder::Get().HasQueuedRecordings();
}

bool SSequenceRecorder::IsRecordVisible() const
{
	return !FSequenceRecorder::Get().IsRecording() && !FAnimationRecorderManager::Get().IsRecording() && !FSequenceRecorder::Get().IsDelaying();
}

void SSequenceRecorder::HandleStopAll()
{
	FSequenceRecorder::Get().StopRecording();
}

bool SSequenceRecorder::CanStopAll() const
{
	return FSequenceRecorder::Get().IsRecording() || FAnimationRecorderManager::Get().IsRecording() || FSequenceRecorder::Get().IsDelaying();
}

bool SSequenceRecorder::IsStopAllVisible() const
{
	return FSequenceRecorder::Get().IsRecording() || FAnimationRecorderManager::Get().IsRecording() || FSequenceRecorder::Get().IsDelaying();
}

void SSequenceRecorder::HandleAddRecording()
{
	FSequenceRecorder::Get().AddNewQueuedRecordingsForSelectedActors();
}

bool SSequenceRecorder::CanAddRecording() const
{
	return !FAnimationRecorderManager::Get().IsRecording();
}

void SSequenceRecorder::HandleAddCurrentPlayerRecording()
{
	FSequenceRecorder::Get().AddNewQueuedRecordingForCurrentPlayer();
}

bool SSequenceRecorder::CanAddCurrentPlayerRecording() const
{
	return FSequenceRecorder::Get().CanAddNewQueuedRecordingForCurrentPlayer();
}

void SSequenceRecorder::HandleRemoveRecording()
{
	TArray<UActorRecording*> SelectedActorRecordings;
	ActorListView->GetSelectedItems(SelectedActorRecordings);
	UActorRecording* SelectedActorRecording = SelectedActorRecordings.Num() > 0 ? SelectedActorRecordings[0] : nullptr;

	if(SelectedActorRecording)
	{
		FSequenceRecorder::Get().RemoveQueuedRecording(SelectedActorRecording);

		// Remove the recording from the current group here. We can't use the
		// FSequenceRecorder function as they are called when switching groups,
		// and not just when the user removes items.
		if (FSequenceRecorder::Get().GetCurrentRecordingGroup().IsValid())
		{
			FSequenceRecorder::Get().GetCurrentRecordingGroup()->RecordedActors.Remove(SelectedActorRecording);
		}

		TArray<TWeakObjectPtr<UObject>> SelectedObjects = SelectedRecordingItemDetailsView->GetSelectedObjects();
		if(SelectedObjects.Num() > 0 && SelectedObjects[0].Get() == SelectedActorRecording)
		{
			SelectedRecordingItemDetailsView->SetObject(nullptr);
		}
	}
	else
	{
		for (TSharedPtr<SListView<USequenceRecordingBase*>>& ListView : ExtenderListViews)
		{
			TArray<USequenceRecordingBase*> SelectedBaseRecordings;
			ListView->GetSelectedItems(SelectedBaseRecordings);

			USequenceRecordingBase* SelectedBaseRecording = SelectedBaseRecordings.Num() > 0 ? SelectedBaseRecordings[0] : nullptr;
			if (SelectedBaseRecording)
			{
				FSequenceRecorder::Get().RemoveQueuedRecording(SelectedBaseRecording);

				TArray<TWeakObjectPtr<UObject>> SelectedObjects = SelectedRecordingItemDetailsView->GetSelectedObjects();
				if (SelectedObjects.Num() > 0 && SelectedObjects[0].Get() == SelectedBaseRecording)
				{
					SelectedRecordingItemDetailsView->SetObject(nullptr);
					break;
				}
			}
		}
	}
}

bool SSequenceRecorder::CanRemoveRecording() const
{
	int32 NumItemsSelected = 0;
	for (const TSharedPtr<SListView<USequenceRecordingBase*>>& ListView : ExtenderListViews)
	{
		TArray<USequenceRecordingBase*> SelectedBaseRecordings;
		ListView->GetSelectedItems(SelectedBaseRecordings);

		for (USequenceRecordingBase* SelectedBaseRecording : SelectedBaseRecordings)
		{
			if (SelectedBaseRecording)
			{
				++NumItemsSelected;
			}
		}
	}

	TArray<UActorRecording*> SelectedActorRecordings;
	ActorListView->GetSelectedItems(SelectedActorRecordings);

	for (UActorRecording* ActorRecording : SelectedActorRecordings)
	{
		if (ActorRecording)
		{
			++NumItemsSelected;
		}
	}

	return (NumItemsSelected > 0) && !FSequenceRecorder::Get().IsRecording() && !FAnimationRecorderManager::Get().IsRecording();
}

void SSequenceRecorder::HandleRemoveAllRecordings()
{
	FSequenceRecorder::Get().ClearQueuedRecordings();
	if (FSequenceRecorder::Get().GetCurrentRecordingGroup().IsValid())
	{
		FSequenceRecorder::Get().GetCurrentRecordingGroup()->RecordedActors.Empty();
	}
	SelectedRecordingItemDetailsView->SetObject(nullptr);
}

bool SSequenceRecorder::CanRemoveAllRecordings() const
{
	return FSequenceRecorder::Get().HasQueuedRecordings() && !FSequenceRecorder::Get().IsRecording() && !FAnimationRecorderManager::Get().IsRecording();
}

EActiveTimerReturnType SSequenceRecorder::HandleRefreshItems(double InCurrentTime, float InDeltaTime)
{
	if(FSequenceRecorder::Get().AreQueuedRecordingsDirty())
	{
		ActorListView->RequestListRefresh();
		for (TSharedPtr<SListView<USequenceRecordingBase*>>& ListView : ExtenderListViews)
		{
			ListView->RequestListRefresh();
		}
		FSequenceRecorder::Get().ResetQueuedRecordingsDirty();
	}

	return EActiveTimerReturnType::Continue;
}

void SSequenceRecorder::HandleAddRecordingGroup()
{
	TWeakObjectPtr<USequenceRecorderActorGroup> ActorGroup = FSequenceRecorder::Get().AddRecordingGroup();
	check(ActorGroup.IsValid());
}

void SSequenceRecorder::HandleRecordingGroupAddedToSequenceRecorder(TWeakObjectPtr<USequenceRecorderActorGroup> ActorGroup)
{
	if (ActorGroup.IsValid())
	{
		RecordingGroupDetailsView->SetObject(ActorGroup.Get());
	}
	else
	{
		// Fall back to the CDO in the event of an unexpected failure so the UI doesn't disappear.
		RecordingGroupDetailsView->SetObject(GetMutableDefault<USequenceRecorderActorGroup>());
	}
}

bool SSequenceRecorder::CanAddRecordingGroup() const
{
	return !FSequenceRecorder::Get().IsRecording();
}

void SSequenceRecorder::HandleLoadRecordingActorGroup(FName Name)
{
	FSequenceRecorder::Get().LoadRecordingGroup(Name);

	// Bind our details view to the newly loaded group.
	if (FSequenceRecorder::Get().GetCurrentRecordingGroup().IsValid())
	{
		RecordingGroupDetailsView->SetObject(FSequenceRecorder::Get().GetCurrentRecordingGroup().Get());
	}
	else
	{
		// If they've loaded the "None" profile we create a new default as well to reset the paths.
		RecordingGroupDetailsView->SetObject(GetMutableDefault<USequenceRecorderActorGroup>());
	}
}

void SSequenceRecorder::HandleRemoveRecordingGroup()
{
	FSequenceRecorder::Get().RemoveCurrentRecordingGroup();

	// See if there's any recordings left, if so we'll load the last one, otherwise we need to load
	// a default so that the UI is still visible.
	TArray<FName> RecordingProfiles = FSequenceRecorder::Get().GetRecordingGroupNames();
	if (RecordingProfiles.Num() > 0)
	{
		FSequenceRecorder::Get().LoadRecordingGroup(RecordingProfiles[RecordingProfiles.Num() - 1]);
		check(FSequenceRecorder::Get().GetCurrentRecordingGroup().Get());

		RecordingGroupDetailsView->SetObject(FSequenceRecorder::Get().GetCurrentRecordingGroup().Get());
	}
	else
	{
		RecordingGroupDetailsView->SetObject(GetMutableDefault<USequenceRecorderActorGroup>());
	}
}

bool SSequenceRecorder::CanRemoveRecordingGroup() const
{
	TWeakObjectPtr<USequenceRecorderActorGroup> RecordingGroup = FSequenceRecorder::Get().GetCurrentRecordingGroup();
	if (RecordingGroup.IsValid())
	{
		return RecordingGroup->GroupName != NAME_None;
	}

	return false;
}

void SSequenceRecorder::HandleDuplicateRecordingGroup()
{
	FSequenceRecorder::Get().DuplicateRecordingGroup();
}

bool SSequenceRecorder::CanDuplicateRecordingGroup() const
{
	TWeakObjectPtr<USequenceRecorderActorGroup> RecordingGroup = FSequenceRecorder::Get().GetCurrentRecordingGroup();
	if (RecordingGroup.IsValid())
	{
		return RecordingGroup->GroupName != NAME_None;
	}

	return false;
}

TOptional<float> SSequenceRecorder::GetDelayPercent() const
{
	const float Delay = GetDefault<USequenceRecorderSettings>()->RecordingDelay;
	const float Countdown = FSequenceRecorder::Get().GetCurrentDelay();
	return Delay > 0.0f ? Countdown / Delay : 0.0f;
}

EVisibility SSequenceRecorder::GetDelayProgressVisibilty() const
{
	return FSequenceRecorder::Get().IsDelaying() ? EVisibility::Visible : EVisibility::Hidden;
}

FText SSequenceRecorder::GetTargetSequenceName() const
{
	return FText::Format(LOCTEXT("NextSequenceFormat", "Next Sequence: {0}"), FText::FromString(FSequenceRecorder::Get().GetNextSequenceName()));
}

bool SSequenceRecorder::OnRecordingActorListAllowDrop( TSharedPtr<FDragDropOperation> DragDropOperation )
{
	return DragDropOperation->IsOfType<FActorDragDropOp>();
}

FReply SSequenceRecorder::OnRecordingActorListDrop( const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent )
{
	TSharedPtr<FActorDragDropOp> ActorDragDropOperation = InDragDropEvent.GetOperationAs<FActorDragDropOp>();
	if ( ActorDragDropOperation )
	{
		for (auto Actor : ActorDragDropOperation->Actors)
		{
			if (Actor.IsValid())
			{
				FSequenceRecorder::Get().AddNewQueuedRecording(Actor.Get());
			}
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
