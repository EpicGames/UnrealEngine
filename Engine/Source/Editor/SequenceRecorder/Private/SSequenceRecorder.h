// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "ActorRecording.h"
#include "SequenceRecordingBase.h"
#include "PropertyEditorDelegates.h"
#include "ISinglePropertyView.h"
#include "IStructureDetailsView.h"

class FActiveTimerHandle;
class FUICommandList;
class FUICommandInfo;
class IDetailsView;
class SProgressBar;
class FDragDropOperation;
class SEditableTextBox;

class SSequenceRecorder : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SSequenceRecorder)
	{
	}

	SLATE_END_ARGS()

	void Construct(const FArguments& Args);
	
	/** SSequenceRecorder destructor */
	~SSequenceRecorder();

private:

	void BindCommands();

	TSharedRef<ITableRow> MakeListViewWidget(UActorRecording* Recording, const TSharedRef<STableViewBase>& OwnerTable) const;

	FText GetRecordingActorName(UActorRecording* Recording) const;

	void OnActorListSelectionChanged(UActorRecording* Recording, ESelectInfo::Type SelectionType);

	void OnListSelectionChanged(USequenceRecordingBase* Recording);

	void HandleRecord();

	EActiveTimerReturnType StartDelayedRecord(double InCurrentTime, float InDeltaTime);

	bool CanRecord() const;

	bool IsRecordVisible() const;

	void HandleStopAll();

	bool CanStopAll() const;

	bool IsStopAllVisible() const;

	void HandleAddRecording();
	void HandleRecordingGroupAddedToSequenceRecorder(TWeakObjectPtr<class USequenceRecorderActorGroup> ActorGroup);

	bool CanAddRecording() const;

	void HandleAddCurrentPlayerRecording();

	bool CanAddCurrentPlayerRecording() const;

	void HandleRemoveRecording();

	bool CanRemoveRecording() const;

	void HandleRemoveAllRecordings();

	bool CanRemoveAllRecordings() const;

	void HandleAddRecordingGroup();


	bool CanAddRecordingGroup() const;

	void HandleRemoveRecordingGroup();

	bool CanRemoveRecordingGroup() const;

	void HandleDuplicateRecordingGroup();

	bool CanDuplicateRecordingGroup() const;

	void HandleRecordingProfileNameCommitted(const FText& InText, ETextCommit::Type InCommitType);

	EActiveTimerReturnType HandleRefreshItems(double InCurrentTime, float InDeltaTime);

	void HandleMapUnload(UObject* Object);

	TOptional<float> GetDelayPercent() const;

	void OnDelayChanged(float NewValue);

	EVisibility GetDelayProgressVisibilty() const;

	FText GetTargetSequenceName() const;

	FReply OnRecordingActorListDrop( const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent );

	bool OnRecordingActorListAllowDrop( TSharedPtr<FDragDropOperation> DragDropOperation );

public:
	TSharedPtr<FUICommandList> GetCommandList() const
	{
		return CommandList;
	}

	void HandleLoadRecordingActorGroup(FName Name);

private:
	/** This is the Detail View for the USequenceRecorderSettings */
	TSharedPtr<IDetailsView> SequenceRecordingDetailsView;

	/** This is the Detail View for the currently selected UActorRecording or an item from an Extender */
	TSharedPtr<IDetailsView> SelectedRecordingItemDetailsView;

	/** This is the Detail View for the currently selected USequenceActorGroup */
	TSharedPtr<IDetailsView> RecordingGroupDetailsView;

	TSharedPtr<SListView<UActorRecording*>> ActorListView;

	/** List of all the the Detail View created by Extenders */
	TArray<TSharedPtr<SListView<USequenceRecordingBase*>>> ExtenderListViews;

	TSharedPtr<FUICommandList> CommandList;

	/** The handle to the refresh tick timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	TSharedPtr<SProgressBar> DelayProgressBar;

	mutable bool bInsideSelectionChanged;
};
