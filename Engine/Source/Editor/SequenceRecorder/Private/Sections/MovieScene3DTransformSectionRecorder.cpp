// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieScene3DTransformSectionRecorder.h"
#include "Animation/Skeleton.h"
#include "Misc/ScopedSlowTask.h"
#include "GameFramework/Character.h"
#include "KeyParams.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "SequenceRecorder.h"
#include "SequenceRecorderSettings.h"
#include "SequenceRecorderUtils.h"
#include "Algo/Transform.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/AnimSequence.h"
#include "Evaluation/MovieSceneEvaluationState.h"

TSharedPtr<IMovieSceneSectionRecorder> FMovieScene3DTransformSectionRecorderFactory::CreateSectionRecorder(const FActorRecordingSettings& InActorRecordingSettings) const
{
	return nullptr;
}

TSharedPtr<FMovieScene3DTransformSectionRecorder> FMovieScene3DTransformSectionRecorderFactory::CreateSectionRecorder(bool bRecordTransforms, TSharedPtr<class FMovieSceneAnimationSectionRecorder> InAnimRecorder) const
{
	return MakeShareable(new FMovieScene3DTransformSectionRecorder(bRecordTransforms, InAnimRecorder));
}

bool FMovieScene3DTransformSectionRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	if(USceneComponent* SceneComponent = Cast<USceneComponent>(InObjectToRecord))
	{
		// Dont record the root component transforms as this will be taken into account by the actor transform track
		// Also dont record transforms of skeletal mesh components as they will be taken into account in the actor transform
		bool bIsCharacterSkelMesh = false;
		if (SceneComponent->IsA<USkeletalMeshComponent>() && SceneComponent->GetOwner()->IsA<ACharacter>())
		{
			ACharacter* Character = CastChecked<ACharacter>(SceneComponent->GetOwner());
			bIsCharacterSkelMesh = SceneComponent == Character->GetMesh();
		}

		return (SceneComponent != SceneComponent->GetOwner()->GetRootComponent() && !bIsCharacterSkelMesh);
	}
	else 
	{
		return InObjectToRecord->IsA<AActor>();
	}
}

void FMovieScene3DTransformSectionRecorder::CreateSection(UObject* InObjectToRecord, UMovieScene* InMovieScene, const FGuid& InGuid, float Time) 
{
	ObjectToRecord = InObjectToRecord;
	Guid = InGuid;
	bWasAttached = false;
	RecordingStartTime = Time;

	MovieScene = InMovieScene;

	static FName Transform("Transform");
	MovieSceneTrack = InMovieScene->FindTrack<UMovieScene3DTransformTrack>(Guid, Transform);
	if (!MovieSceneTrack.IsValid())
	{
		MovieSceneTrack = InMovieScene->AddTrack<UMovieScene3DTransformTrack>(Guid);
	}
	else
	{
		MovieSceneTrack->RemoveAllAnimationData();
	}

	if(MovieSceneTrack.IsValid())
	{
		MovieSceneSection = Cast<UMovieScene3DTransformSection>(MovieSceneTrack->CreateNewSection());

		MovieSceneTrack->AddSection(*MovieSceneSection);

		DefaultTransform = FTransform::Identity;
		GetTransformToRecord(DefaultTransform);

		FVector Translation   = DefaultTransform.GetTranslation();
		FVector EulerRotation = DefaultTransform.GetRotation().Rotator().Euler();
		FVector Scale         = DefaultTransform.GetScale3D();

		TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = MovieSceneSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
		DoubleChannels[0]->SetDefault(Translation.X);
		DoubleChannels[1]->SetDefault(Translation.Y);
		DoubleChannels[2]->SetDefault(Translation.Z);
		DoubleChannels[3]->SetDefault(EulerRotation.X);
		DoubleChannels[4]->SetDefault(EulerRotation.Y);
		DoubleChannels[5]->SetDefault(EulerRotation.Z);
		DoubleChannels[6]->SetDefault(Scale.X);
		DoubleChannels[7]->SetDefault(Scale.Y);
		DoubleChannels[8]->SetDefault(Scale.Z);

		FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame    = (Time * TickResolution).FloorToFrame();

		MovieSceneSection->SetRange(TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame));

		MovieSceneSection->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();
	}
}

void FMovieScene3DTransformSectionRecorder::FinalizeSection(float CurrentTime)
{
	if (!MovieSceneSection.IsValid())
	{
		return;
	}

	bool bWasRecording = bRecording;
	bRecording = false;

	FScopedSlowTask SlowTask(4.0f, NSLOCTEXT("SequenceRecorder", "ProcessingTransforms", "Processing Transforms"));

	check (	BufferedTransforms.Times.Num() == BufferedTransforms.LocationX.Num());	
	check (	BufferedTransforms.Times.Num() == BufferedTransforms.LocationY.Num());
	check (	BufferedTransforms.Times.Num() == BufferedTransforms.LocationZ.Num());
	check (	BufferedTransforms.Times.Num() == BufferedTransforms.RotationX.Num());
	check (	BufferedTransforms.Times.Num() == BufferedTransforms.RotationY.Num());
	check (	BufferedTransforms.Times.Num() == BufferedTransforms.RotationZ.Num());
	check (	BufferedTransforms.Times.Num() == BufferedTransforms.ScaleX.Num());
	check (	BufferedTransforms.Times.Num() == BufferedTransforms.ScaleY.Num());
	check (	BufferedTransforms.Times.Num() == BufferedTransforms.ScaleZ.Num());

	// if we have a valid animation recorder, we need to build our transforms from the animation
	// so we properly synchronize our keyframes. This should only be done when recording animation 
	// to the animation asset in world space because otherwise if recording to local space, the root 
	// bone would resolve to identity and local space transform keys would be recorded.
	if(AnimRecorder.IsValid() && bWasRecording && AnimRecorder->AnimationSettings.bRecordInWorldSpace)
	{
		check(BufferedTransforms.Times.Num() == 0);

		UAnimSequence* AnimSequence = AnimRecorder->GetAnimSequence();
		USkeletalMeshComponent* SkeletalMeshComponent = AnimRecorder->GetSkeletalMeshComponent();
		if (SkeletalMeshComponent)
		{
			USkinnedAsset* SkinnedAsset =
				SkeletalMeshComponent->LeaderPoseComponent != nullptr ?
				SkeletalMeshComponent->LeaderPoseComponent->GetSkinnedAsset() :
				SkeletalMeshComponent->GetSkinnedAsset();
			if (AnimSequence && SkinnedAsset)
			{
				// find the root bone
				int32 RootIndex = INDEX_NONE;
				USkeleton* AnimSkeleton = AnimSequence->GetSkeleton();

				TArray<FName> TrackNames;
				AnimSequence->GetDataModelInterface()->GetBoneTrackNames(TrackNames);

				FName RootBoneName = NAME_None;
				for (const FName& TrackName : TrackNames)
				{
					// Verify if this bone exists in skeleton
					const int32 SkeletonTreeIndex = AnimSkeleton->GetReferenceSkeleton().FindBoneIndex(TrackName);
					if (SkeletonTreeIndex != INDEX_NONE)
					{
						const int32 BoneIndex = AnimSkeleton->GetMeshBoneIndexFromSkeletonBoneIndex(SkinnedAsset, SkeletonTreeIndex);
						const int32 ParentIndex = SkinnedAsset->GetRefSkeleton().GetParentIndex(BoneIndex);
						if (ParentIndex == INDEX_NONE)
						{
							// We've found the root (root bones do not have a valid parent)
							RootIndex = BoneIndex;
							RootBoneName = TrackName;
							break;
						}
					}
				}
				check(RootBoneName != NAME_None);

				FFrameRate TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
				const FFrameNumber StartTime = (RecordingStartTime * TickResolution).FloorToFrame();

				// we may need to offset the transform here if the animation was not recorded on the root component
				FTransform InvComponentTransform = AnimRecorder->GetComponentTransform().Inverse();

				for (int32 KeyIndex = 0; KeyIndex < AnimSequence->GetDataModelInterface()->GetNumberOfKeys(); KeyIndex++)
				{
					const FTransform Transform = AnimSequence->GetDataModelInterface()->GetBoneTrackTransform(RootBoneName, FFrameNumber(KeyIndex));
					FFrameNumber AnimationFrame = (AnimSequence->GetTimeAtFrame(KeyIndex) * TickResolution).FloorToFrame();
					BufferedTransforms.Add(InvComponentTransform * Transform, StartTime + AnimationFrame);
				}
			}
		}
	}

	SlowTask.EnterProgressFrame();

	// Try to 're-wind' rotations that look like axis flips
	// We need to do this as a post-process because the recorder cant reliably access 'wound' rotations:
	// - Net quantize may use quaternions.
	// - Scene components cache transforms as quaternions.
	// - Gameplay is free to clamp/fmod rotations as it sees fit.
	int32 TransformCount = BufferedTransforms.Times.Num();
	for(int32 TransformIndex = 0; TransformIndex < TransformCount - 1; TransformIndex++)
	{
		FMath::WindRelativeAnglesDegrees(BufferedTransforms.RotationZ[TransformIndex], BufferedTransforms.RotationZ[TransformIndex+1]);
		FMath::WindRelativeAnglesDegrees(BufferedTransforms.RotationY[TransformIndex], BufferedTransforms.RotationY[TransformIndex+1]);
		FMath::WindRelativeAnglesDegrees(BufferedTransforms.RotationX[TransformIndex], BufferedTransforms.RotationX[TransformIndex+1]);
	}

	SlowTask.EnterProgressFrame();

	// If we are syncing to an animation, use linear interpolation to avoid foot sliding etc. 
	// Otherwise use cubic for better quality (much better for projectiles etc.)

	// @todo: sequencer-timecode: this was previously never actually used - should it be??
	const ERichCurveInterpMode Interpolation = AnimRecorder.IsValid() ? RCIM_Linear : RCIM_Cubic;

	// add buffered transforms
	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = MovieSceneSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

	auto Transformation = [Interpolation](double In)
	{
		FMovieSceneDoubleValue NewValue(In);
		NewValue.InterpMode = Interpolation;
		return NewValue;
	};
	TArray<FMovieSceneDoubleValue> DoubleValues;

	DoubleValues.Reset(BufferedTransforms.LocationX.Num());
	Algo::Transform(BufferedTransforms.LocationX, DoubleValues, Transformation);
	DoubleChannels[0]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));

	DoubleValues.Reset(BufferedTransforms.LocationY.Num());
	Algo::Transform(BufferedTransforms.LocationY, DoubleValues, Transformation);
	DoubleChannels[1]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));

	DoubleValues.Reset(BufferedTransforms.LocationZ.Num());
	Algo::Transform(BufferedTransforms.LocationZ, DoubleValues, Transformation);
	DoubleChannels[2]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));

	DoubleValues.Reset(BufferedTransforms.RotationX.Num());
	Algo::Transform(BufferedTransforms.RotationX, DoubleValues, Transformation);
	DoubleChannels[3]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));

	DoubleValues.Reset(BufferedTransforms.RotationY.Num());
	Algo::Transform(BufferedTransforms.RotationY, DoubleValues, Transformation);
	DoubleChannels[4]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));

	DoubleValues.Reset(BufferedTransforms.RotationZ.Num());
	Algo::Transform(BufferedTransforms.RotationZ, DoubleValues, Transformation);
	DoubleChannels[5]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));

	DoubleValues.Reset(BufferedTransforms.ScaleX.Num());
	Algo::Transform(BufferedTransforms.ScaleX, DoubleValues, Transformation);
	DoubleChannels[6]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));

	DoubleValues.Reset(BufferedTransforms.ScaleY.Num());
	Algo::Transform(BufferedTransforms.ScaleY, DoubleValues, Transformation);
	DoubleChannels[7]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));

	DoubleValues.Reset(BufferedTransforms.ScaleZ.Num());
	Algo::Transform(BufferedTransforms.ScaleZ, DoubleValues, Transformation);
	DoubleChannels[8]->Set(BufferedTransforms.Times, MoveTemp(DoubleValues));

	FTransform FirstTransform = FTransform::Identity;
	if (BufferedTransforms.Times.Num())
	{
		FirstTransform.SetTranslation(FVector(BufferedTransforms.LocationX[0], BufferedTransforms.LocationY[0], BufferedTransforms.LocationZ[0]));
		FirstTransform.SetRotation(FQuat(FRotator(BufferedTransforms.RotationY[0], BufferedTransforms.RotationZ[0], BufferedTransforms.RotationX[0])));
		FirstTransform.SetScale3D(FVector(BufferedTransforms.ScaleX[0], BufferedTransforms.ScaleY[0], BufferedTransforms.ScaleZ[0]));
	}

	BufferedTransforms = FBufferedTransformKeys();

	SlowTask.EnterProgressFrame();

	// now remove linear keys
	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();
	if (Settings->bReduceKeys)
	{
		FKeyDataOptimizationParams Params;

		for (FMovieSceneDoubleChannel* Channel : DoubleChannels)
		{
			Channel->Optimize(Params);
		}
	}
	else
	{
		for (FMovieSceneDoubleChannel* Channel : DoubleChannels)
		{
			Channel->AutoSetTangents();
		}
	}

	// we cant remove redundant tracks if we were attached as the playback relies on update order of
	// transform tracks. Without this track, relative transforms would accumulate.
	if(!bWasAttached)
	{
		bool bCanRemoveTrack = true;
		for (FMovieSceneDoubleChannel* Channel : MovieSceneSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>())
		{
			if (Channel)
			{
				int32 NumKeys = Channel->GetTimes().Num();
				if (NumKeys == 1)
				{
					*Channel = FMovieSceneDoubleChannel();
				}
				else if (NumKeys > 1)
				{
					bCanRemoveTrack = false;
				}
			}
		}

		if (bCanRemoveTrack)
		{
			if(DefaultTransform.Equals(FTransform::Identity))
			{
				MovieScene->RemoveTrack(*MovieSceneTrack.Get());
			}
		}
	}

	SlowTask.EnterProgressFrame();

	// If recording a spawnable, update the spawnable object template to the first keyframe
	if (MovieScene.IsValid() && Guid.IsValid())
	{
		UMovieSceneSequence* ThisSequence = MovieScene->GetTypedOuter<UMovieSceneSequence>();
		TSharedRef<UE::MovieScene::FSharedPlaybackState> TransientPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(GetSourceObject(), ThisSequence);

		if (AActor* ActorTemplatePtr = Cast<AActor>(MovieSceneHelpers::GetObjectTemplate(ThisSequence, Guid, TransientPlaybackState, 0)))
		{
			if (USceneComponent* RootComponent = ActorTemplatePtr->GetRootComponent())
			{
				RootComponent->SetRelativeTransform(FirstTransform);
			}
		}
	}

	FFrameRate   TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	
	FFrameNumber CurrentFrame    = (CurrentTime * TickResolution).FloorToFrame();
	
	MovieSceneSection->ExpandToFrame(CurrentFrame);
}

void FMovieScene3DTransformSectionRecorder::Record(float CurrentTime)
{
	if (!MovieSceneSection.IsValid())
	{
		return;
	}

	if(ObjectToRecord.IsValid())
	{
		if(USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectToRecord.Get()))
		{
			// dont record non-registered scene components
			if(!SceneComponent->IsRegistered())
			{
				return;
			}
		}

		FFrameRate   TickResolution  = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame    = (CurrentTime * TickResolution).FloorToFrame();

		if(bRecording)
		{
			// don't record from the transform of the component/actor if we are synchronizing with an animation
			if (!AnimRecorder.IsValid() || !AnimRecorder->AnimationSettings.bRecordInWorldSpace)
			{
				FTransform TransformToRecord;
				if (GetTransformToRecord(TransformToRecord))
				{
					BufferedTransforms.Add(TransformToRecord, CurrentFrame);
				}
			}
		}
	}
}

bool FMovieScene3DTransformSectionRecorder::GetTransformToRecord(FTransform& TransformToRecord)
{
	if(USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectToRecord.Get()))
	{
		TransformToRecord = SceneComponent->GetRelativeTransform();
		return true;
	}
	else if(AActor* Actor = Cast<AActor>(ObjectToRecord.Get()))
	{
		bool bCaptureWorldSpaceTransform = false;

		USceneComponent* RootComponent = Actor->GetRootComponent();
		USceneComponent* AttachParent = RootComponent ? RootComponent->GetAttachParent() : nullptr;

		bWasAttached = AttachParent != nullptr;
		if (AttachParent)
		{
			// We capture world space transforms for actors if they're attached, but we're not recording the attachment parent
			bCaptureWorldSpaceTransform = !FSequenceRecorder::Get().FindRecording(AttachParent->GetOwner());
		}

		if (!RootComponent)
		{
			return false;
		}

		if (bCaptureWorldSpaceTransform)
		{
			TransformToRecord = Actor->ActorToWorld();
			return true;
		}
		else
		{
			TransformToRecord = RootComponent->GetRelativeTransform();
			return true;
		}
	}

	return false;
}
