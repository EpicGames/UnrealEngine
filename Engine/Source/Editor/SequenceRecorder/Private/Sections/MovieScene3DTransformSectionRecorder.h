// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMovieSceneSectionRecorder.h"
#include "IMovieSceneSectionRecorderFactory.h"
#include "Sections/MovieSceneAnimationSectionRecorder.h"
#include "MovieScene.h"
#include "Sections/MovieScene3DTransformSectionRecorderSettings.h"

class FMovieScene3DTransformSectionRecorder;
class UMovieScene3DTransformSection;
class UMovieScene3DTransformTrack;
struct FActorRecordingSettings;

class FMovieScene3DTransformSectionRecorderFactory : public IMovieSceneSectionRecorderFactory
{
public:
	virtual ~FMovieScene3DTransformSectionRecorderFactory() = default;

	virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
	virtual UObject* CreateSettingsObject(class UObject* InOuter) const override { return NewObject<UMovieScene3DTransformSectionRecorderSettings>(InOuter, FName(TEXT("MovieScene3DTransformSectionRecorderSettings"))); }

	TSharedPtr<class FMovieScene3DTransformSectionRecorder> CreateSectionRecorder(bool bRecordTransforms, TSharedPtr<class FMovieSceneAnimationSectionRecorder> InAnimRecorder) const;

private:
	virtual TSharedPtr<IMovieSceneSectionRecorder> CreateSectionRecorder(const FActorRecordingSettings& InActorRecordingSettings) const override;
};

/** Structure used to buffer up transform keys. Keys are inserted into tracks in FinalizeSection() */
struct FBufferedTransformKeys
{
	void Add(const FTransform& InTransform, FFrameNumber InKeyTime)
	{
		Times.Add(InKeyTime);
		LocationX.Add(InTransform.GetTranslation().X);
		LocationY.Add(InTransform.GetTranslation().Y);
		LocationZ.Add(InTransform.GetTranslation().Z);

		FRotator WoundRoation = InTransform.Rotator();
		RotationX.Add(WoundRoation.Roll);
		RotationY.Add(WoundRoation.Pitch);
		RotationZ.Add(WoundRoation.Yaw);

		ScaleX.Add(InTransform.GetScale3D().X);
		ScaleY.Add(InTransform.GetScale3D().Y);
		ScaleZ.Add(InTransform.GetScale3D().Z);
	}

	TArray<FFrameNumber> Times;
	TArray<float> LocationX, LocationY, LocationZ;
	TArray<float> RotationX, RotationY, RotationZ;
	TArray<float> ScaleX, ScaleY, ScaleZ;
};

class FMovieScene3DTransformSectionRecorder : public IMovieSceneSectionRecorder
{
public:
	FMovieScene3DTransformSectionRecorder(bool bInActuallyRecord, TSharedPtr<class FMovieSceneAnimationSectionRecorder> InAnimRecorder = nullptr) 
		: bRecording(bInActuallyRecord)
		, AnimRecorder(InAnimRecorder)
	{}

	virtual ~FMovieScene3DTransformSectionRecorder() = default;

	virtual void CreateSection(UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& Guid, float Time) override;
	virtual void FinalizeSection(float CurrentTime) override;
	virtual void Record(float CurrentTime) override;
	virtual void InvalidateObjectToRecord() override
	{
		ObjectToRecord = nullptr;
	}
	virtual UObject* GetSourceObject() const override
	{
		return ObjectToRecord.Get();
	}

private:
	bool GetTransformToRecord(FTransform& TransformToRecord);

private:
	/** Object to record from */
	TWeakObjectPtr<UObject> ObjectToRecord;

	/** MovieScene to record to */
	TWeakObjectPtr<class UMovieScene> MovieScene;

	/** Track to record to */
	TWeakObjectPtr<class UMovieScene3DTransformTrack> MovieSceneTrack;

	/** Section to record to */
	TWeakObjectPtr<class UMovieScene3DTransformSection> MovieSceneSection;

	/** Buffer of transform keys. Keys are inserted into tracks in FinalizeSection() */
	FBufferedTransformKeys BufferedTransforms;

	/** Flag if we are actually recording or not */
	bool bRecording;

	/** Animation recorder we use to sync our transforms to */
	TSharedPtr<class FMovieSceneAnimationSectionRecorder> AnimRecorder;

	/** The default transform this recording starts with */
	FTransform DefaultTransform;

	/** Flag indicating that some time while this recorder was active an attachment was also in place */
	bool bWasAttached;

	/** The guid being recorded to */
	FGuid Guid;

	/** The guid being recorded to */
	float RecordingStartTime;
};
