// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "MovieScene3DTransformSectionRecorderSettings.generated.h"

UCLASS()
class UMovieScene3DTransformSectionRecorderSettings : public UObject
{
public:
	GENERATED_BODY()

	/**
	 * Whether to record actor transforms. This can be useful if you want the actors to end up in specific locations after the sequence.
	 * By default we rely on animations to provide transforms, but this can be changed using the "Record In World Space" animation setting.
	 */
	UPROPERTY(EditAnywhere, Category = "Actor Recording")
	bool bRecordTransforms;
};
