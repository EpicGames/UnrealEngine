// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneAnimationSectionRecorder.h"
#include "AnimationRecorder.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "MovieScene.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "SequenceRecorderUtils.h"
#include "SequenceRecorderSettings.h"
#include "ActorRecording.h"
#include "SequenceRecorder.h"

TSharedPtr<IMovieSceneSectionRecorder> FMovieSceneAnimationSectionRecorderFactory::CreateSectionRecorder(const FActorRecordingSettings& InActorRecordingSettings) const
{
	return nullptr;
}

TSharedPtr<FMovieSceneAnimationSectionRecorder> FMovieSceneAnimationSectionRecorderFactory::CreateSectionRecorder(UAnimSequence* InAnimSequence, const FAnimationRecordingSettings& InAnimationSettings, const FString& InAnimAssetPath, const FString& InAnimAssetName) const
{
	return MakeShareable(new FMovieSceneAnimationSectionRecorder(InAnimationSettings, InAnimSequence, InAnimAssetPath, InAnimAssetName));
}

bool FMovieSceneAnimationSectionRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	if (InObjectToRecord->IsA<USkeletalMeshComponent>())
	{
		USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InObjectToRecord);
		if (SkeletalMeshComponent && SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			return true;
		}
	}
	return false;
}

FMovieSceneAnimationSectionRecorder::FMovieSceneAnimationSectionRecorder(const FAnimationRecordingSettings& InAnimationSettings, UAnimSequence* InSpecifiedSequence, const FString& InAnimAssetPath, const FString& InAnimAssetName)
	: AnimationSettings(InAnimationSettings)
	, AnimSequence(InSpecifiedSequence)
	, bRemoveRootTransform(true)
	, AnimAssetPath(InAnimAssetPath)
	, AnimAssetName(InAnimAssetName)
{
}

void FMovieSceneAnimationSectionRecorder::CreateSection(UObject* InObjectToRecord, UMovieScene* MovieScene, const FGuid& Guid, float Time)
{
	ObjectToRecord = InObjectToRecord;

	AActor* Actor = nullptr;
	SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InObjectToRecord);
	if(!SkeletalMeshComponent.IsValid())
	{
		Actor = Cast<AActor>(InObjectToRecord);
		if(Actor != nullptr)
		{
			SkeletalMeshComponent = Actor->FindComponentByClass<USkeletalMeshComponent>();
		}
	}
	else
	{
		Actor = SkeletalMeshComponent->GetOwner();
	}

	if(SkeletalMeshComponent.IsValid())
	{
		SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (SkeletalMesh != nullptr)
		{
			ComponentTransform = SkeletalMeshComponent->GetComponentToWorld().GetRelativeTransform(SkeletalMeshComponent->GetOwner()->GetTransform());

			if (!AnimSequence.IsValid())
			{
				// build an asset path
				const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

				if (AnimAssetPath.IsEmpty())
				{
					AnimAssetPath = FSequenceRecorder::Get().GetSequenceRecordingBasePath();
					if (Settings->AnimationSubDirectory.Len() > 0)
					{
						AnimAssetPath /= Settings->AnimationSubDirectory;
					}
				}

				if (AnimAssetName.IsEmpty())
				{
					const FString SequenceName = FSequenceRecorder::Get().GetSequenceRecordingName();
					AnimAssetName = SequenceName.Len() > 0 ? SequenceName : TEXT("RecordedSequence");
					AnimAssetName += TEXT("_");
					check(Actor);
					AnimAssetName += Actor->GetActorLabel();
				}

				AnimSequence = SequenceRecorderUtils::MakeNewAsset<UAnimSequence>(AnimAssetPath, AnimAssetName);

				if (AnimSequence.IsValid())
				{
					FAssetRegistryModule::AssetCreated(AnimSequence.Get());

					// set skeleton
					AnimSequence->SetSkeleton(SkeletalMeshComponent->GetSkeletalMeshAsset()->GetSkeleton());
				}
			}

			if (AnimSequence.IsValid())
			{
				FAnimationRecorderManager::Get().RecordAnimation(SkeletalMeshComponent.Get(), AnimSequence.Get(), AnimationSettings);

				if (MovieScene)
				{
					UMovieSceneSkeletalAnimationTrack* AnimTrack = MovieScene->FindTrack<UMovieSceneSkeletalAnimationTrack>(Guid);
					if (!AnimTrack)
					{
						AnimTrack = MovieScene->AddTrack<UMovieSceneSkeletalAnimationTrack>(Guid);
					}
					else
					{
						AnimTrack->RemoveAllAnimationData();
					}

					if (AnimTrack)
					{
						FFrameRate   TickResolution  = AnimTrack->GetTypedOuter<UMovieScene>()->GetTickResolution();
						FFrameNumber CurrentFrame    = (Time * TickResolution).FloorToFrame();

						AnimTrack->AddNewAnimation(CurrentFrame, AnimSequence.Get());
						MovieSceneSection = Cast<UMovieSceneSkeletalAnimationSection>(AnimTrack->GetAllSections()[0]);

						MovieSceneSection->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();
					}
				}
			}
		}
	}
}

void FMovieSceneAnimationSectionRecorder::FinalizeSection(float CurrentTime)
{
	if(AnimSequence.IsValid())
	{
		if (AnimationSettings.bRemoveRootAnimation)
		{
			// enable root motion on the animation
			AnimSequence->bForceRootLock = true;
		}
	}

	if(SkeletalMeshComponent.IsValid())
	{
		// only show a message if we dont have a valid movie section
		const bool bShowMessage = !MovieSceneSection.IsValid();
		FAnimationRecorderManager::Get().StopRecordingAnimation(SkeletalMeshComponent.Get(), bShowMessage);
	}

	if(MovieSceneSection.IsValid() && AnimSequence.IsValid() && MovieSceneSection->HasStartFrame())
	{
		FFrameRate   TickResolution  = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber SequenceLength  = (AnimSequence->GetPlayLength() * TickResolution).FloorToFrame();
		
		MovieSceneSection->SetEndFrame(TRangeBound<FFrameNumber>::Exclusive(MovieSceneSection->GetInclusiveStartFrame() + SequenceLength));
	}
}

void FMovieSceneAnimationSectionRecorder::Record(float CurrentTime)
{
	// The animation recorder does most of the work here

	if(SkeletalMeshComponent.IsValid())
	{
		// re-force updates on as gameplay can sometimes turn these back off!
		SkeletalMeshComponent->bEnableUpdateRateOptimizations = false;
		SkeletalMeshComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	}
}
