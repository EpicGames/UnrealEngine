// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimationRecordingSettings.h"
#include "IMovieSceneSectionRecorder.h"
#include "Components/SkeletalMeshComponent.h"
#include "IMovieSceneSectionRecorderFactory.h"

class FMovieSceneAnimationSectionRecorder;
class UMovieSceneSkeletalAnimationSection;
class UActorRecording;
struct FActorRecordingSettings;

class FMovieSceneAnimationSectionRecorderFactory : public IMovieSceneSectionRecorderFactory
{
public:
	virtual ~FMovieSceneAnimationSectionRecorderFactory() = default;

	virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;

	TSharedPtr<class FMovieSceneAnimationSectionRecorder> CreateSectionRecorder(UAnimSequence* InAnimSequence, const FAnimationRecordingSettings& InAnimationSettings, const FString& InAnimAssetPath = FString(), const FString& InAnimAssetName = FString()) const;

private:
	virtual TSharedPtr<IMovieSceneSectionRecorder> CreateSectionRecorder(const FActorRecordingSettings& InActorRecordingSettings) const override;
};

class FMovieSceneAnimationSectionRecorder : public IMovieSceneSectionRecorder
{
public:
	FMovieSceneAnimationSectionRecorder(const FAnimationRecordingSettings& InAnimationSettings, UAnimSequence* InSpecifiedSequence, const FString& InAnimAssetPath = FString(), const FString& InAnimAssetName = FString());
	virtual ~FMovieSceneAnimationSectionRecorder() = default;

	virtual void CreateSection(UObject* InObjectToRecord, class UMovieScene* MovieScene, const FGuid& Guid, float Time) override;
	virtual void FinalizeSection(float CurrentTime) override;
	virtual void Record(float CurrentTime) override;
	virtual void InvalidateObjectToRecord() override
	{
		ObjectToRecord = nullptr;
	}
	virtual UObject* GetSourceObject() const override
	{
		return ObjectToRecord.Get();
	}

	void SetRemoveRootTransform(bool bInRemoveRootTransform) { bRemoveRootTransform = bInRemoveRootTransform; }

	UAnimSequence* GetAnimSequence() const { return AnimSequence.Get(); }

	USkeletalMesh* GetSkeletalMesh() const { return SkeletalMesh.Get(); }

	USkeletalMeshComponent* GetSkeletalMeshComponent() const { return SkeletalMeshComponent.Get(); }

	const FTransform& GetComponentTransform() const { return ComponentTransform; }

	FAnimationRecordingSettings AnimationSettings;

private:
	/** Object to record from */
	TWeakObjectPtr<UObject> ObjectToRecord;

	/** Section to record to */
	TWeakObjectPtr<class UMovieSceneSkeletalAnimationSection> MovieSceneSection;

	TWeakObjectPtr<class UAnimSequence> AnimSequence;

	TWeakObjectPtr<class USkeletalMeshComponent> SkeletalMeshComponent;

	TWeakObjectPtr<class USkeletalMesh> SkeletalMesh;

	bool bRemoveRootTransform;

	/** Local transform of the component we are recording */
	FTransform ComponentTransform;

	FString AnimAssetPath;

	FString AnimAssetName;
};
