// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneMultiPropertyRecorder.h"

#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieScenePropertyRecorder.h"
#include "SequenceRecorderSettings.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/EnumProperty.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

struct FActorRecordingSettings;
struct FGuid;

TSharedPtr<IMovieSceneSectionRecorder> FMovieSceneMultiPropertyRecorderFactory::CreateSectionRecorder(const FActorRecordingSettings& InActorRecordingSettings) const
{
	return MakeShareable(new FMovieSceneMultiPropertyRecorder());
}

bool FMovieSceneMultiPropertyRecorderFactory::CanRecordObject(UObject* InObjectToRecord) const
{
	return true;
#if 0
	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

	for (const FPropertiesToRecordForClass& PropertiesToRecordForClass : Settings->ClassesAndPropertiesToRecord)
	{
		if (*PropertiesToRecordForClass.Class == InObjectToRecord->GetClass() && PropertiesToRecordForClass.Properties.Num() > 0)
		{
			return true;
		}
	}

	for (const FPropertiesToRecordForActorClass& PropertiesToRecordForClass : Settings->ActorsAndPropertiesToRecord)
	{
		if (*PropertiesToRecordForClass.Class == InObjectToRecord->GetClass() && PropertiesToRecordForClass.Properties.Num() > 0)
		{
			return true;
		}
	}

	return false;
#endif
}

FMovieSceneMultiPropertyRecorder::FMovieSceneMultiPropertyRecorder()
{
}

void FMovieSceneMultiPropertyRecorder::CreateSection(UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& Guid, float Time)
{
	ObjectToRecord = InObjectToRecord;

	// collect all properties to record from classes we are recording
	TArray<FName> PropertiesToRecord;

	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();
	for (const FPropertiesToRecordForClass& PropertiesToRecordForClass : Settings->ClassesAndPropertiesToRecord)
	{
		if (*PropertiesToRecordForClass.Class == InObjectToRecord->GetClass() && PropertiesToRecordForClass.Properties.Num() > 0)
		{
			PropertiesToRecord.Append(PropertiesToRecordForClass.Properties);
		}
	}

	for (const FPropertiesToRecordForActorClass& PropertiesToRecordForClass : Settings->ActorsAndPropertiesToRecord)
	{
		if (*PropertiesToRecordForClass.Class == InObjectToRecord->GetClass() && PropertiesToRecordForClass.Properties.Num() > 0)
		{
			PropertiesToRecord.Append(PropertiesToRecordForClass.Properties);
		}
	}

	// create a recorder for each property name
	for (const FName& PropertyName : PropertiesToRecord)
	{
		FTrackInstancePropertyBindings Binding(PropertyName, PropertyName.ToString());
		FProperty* Property = Binding.GetProperty(*InObjectToRecord);
		if (Property != nullptr)
		{
			if (Property->IsA<FBoolProperty>())
			{
				PropertyRecorders.Add(MakeShareable(new FMovieScenePropertyRecorder<bool>(Binding)));
			}
			else if (Property->IsA<FByteProperty>())
			{
				PropertyRecorders.Add(MakeShareable(new FMovieScenePropertyRecorder<uint8>(Binding)));
			}
			else if (Property->IsA<FEnumProperty>())
			{
				PropertyRecorders.Add(MakeShareable(new FMovieScenePropertyRecorderEnum(Binding)));
			}
			else if (Property->IsA<FFloatProperty>())
			{
				PropertyRecorders.Add(MakeShareable(new FMovieScenePropertyRecorder<float>(Binding)));
			}
			else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				if (StructProperty->Struct->GetFName() == NAME_Vector)
				{
					PropertyRecorders.Add(MakeShareable(new FMovieScenePropertyRecorder<FVector>(Binding)));
				}
				else if (StructProperty->Struct->GetFName() == NAME_Color)
				{
					PropertyRecorders.Add(MakeShareable(new FMovieScenePropertyRecorder<FColor>(Binding)));
				}
			}

			PropertyRecorders.Last()->Create(InObjectToRecord, InMovieScene, Guid, Time);
		}
	}
}

void FMovieSceneMultiPropertyRecorder::FinalizeSection(float CurrentTime)
{
	for (TSharedPtr<IMovieScenePropertyRecorder> PropertyRecorder : PropertyRecorders)
	{
		PropertyRecorder->Finalize(ObjectToRecord.Get(), CurrentTime);
	}
}

void FMovieSceneMultiPropertyRecorder::Record(float CurrentTime)
{
	for (TSharedPtr<IMovieScenePropertyRecorder> PropertyRecorder : PropertyRecorders)
	{
		PropertyRecorder->Record(ObjectToRecord.Get(), CurrentTime);
	}
}

bool FMovieSceneMultiPropertyRecorder::CanPropertyBeRecorded(const FProperty& InProperty)
{
	if (InProperty.IsA<FBoolProperty>())
	{
		return true;
	}
	else if (InProperty.IsA<FByteProperty>())
	{
		return true;
	}
	else if (InProperty.IsA<FEnumProperty>())
	{
		return true;
	}
	else if (InProperty.IsA<FFloatProperty>())
	{
		return true;
	}
	else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(&InProperty))
	{
		if (StructProperty->Struct->GetFName() == NAME_Vector)
		{
			return true;
		}
		else if (StructProperty->Struct->GetFName() == NAME_Color)
		{
			return true;
		}
	}

	return false;
}
