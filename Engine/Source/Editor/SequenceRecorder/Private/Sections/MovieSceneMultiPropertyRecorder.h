// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IMovieSceneSectionRecorder.h"
#include "IMovieSceneSectionRecorderFactory.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/Object.h"

class FProperty;
struct FGuid;

class FMovieSceneMultiPropertyRecorderFactory : public IMovieSceneSectionRecorderFactory
{
public:
	virtual ~FMovieSceneMultiPropertyRecorderFactory() = default;

	virtual TSharedPtr<IMovieSceneSectionRecorder> CreateSectionRecorder(const struct FActorRecordingSettings& InActorRecordingSettings) const override;
	virtual bool CanRecordObject(UObject* InObjectToRecord) const override;
};

class FMovieSceneMultiPropertyRecorder : public IMovieSceneSectionRecorder
{
public:
	FMovieSceneMultiPropertyRecorder();
	virtual ~FMovieSceneMultiPropertyRecorder() = default;

	/** IMovieSceneSectionRecorder interface */
	virtual void CreateSection(UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& Guid, float Time) override;
	virtual void FinalizeSection(float CurrentTime) override;
	virtual void Record(float CurrentTime) override;
	virtual void InvalidateObjectToRecord() override
	{
		ObjectToRecord = nullptr;
	}
	virtual UObject* GetSourceObject() const override
	{
		return ObjectToRecord.Get();
	}

	/** Check if the property can be recorded */
	static bool CanPropertyBeRecorded(const FProperty& InProperty);

private:
	/** Object to record from */
	TWeakObjectPtr<class UObject> ObjectToRecord;

	/** All of our property recorders */
	TArray<TSharedPtr<class IMovieScenePropertyRecorder>> PropertyRecorders;
};
