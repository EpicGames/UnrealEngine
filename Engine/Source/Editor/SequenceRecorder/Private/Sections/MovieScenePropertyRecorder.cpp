// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScenePropertyRecorder.h"

#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Curves/RichCurve.h"
#include "Math/MathFwd.h"
#include "Math/Range.h"
#include "Misc/AssertionMacros.h"
#include "MovieScene.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Sections/MovieSceneByteSection.h"
#include "Sections/MovieSceneColorSection.h"
#include "Sections/MovieSceneEnumSection.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Sections/MovieSceneVectorSection.h"
#include "SequenceRecorderUtils.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "Tracks/MovieSceneBoolTrack.h"
#include "Tracks/MovieSceneByteTrack.h"
#include "Tracks/MovieSceneColorTrack.h"
#include "Tracks/MovieSceneEnumTrack.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieSceneVectorTrack.h"
#include "UObject/NameTypes.h"

// current set of compiled-in property types

template <>
bool FMovieScenePropertyRecorder<bool>::ShouldAddNewKey(const bool& InNewValue) const
{
	return InNewValue != PreviousValue;
}

template <>
UMovieSceneSection* FMovieScenePropertyRecorder<bool>::AddSection(UObject* InObjectToRecord, UMovieScene* InMovieScene, const FGuid& InGuid, float InTime)
{
	if (!InObjectToRecord)
	{
		return nullptr;
	}

	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneBoolTrack* Track = InMovieScene->FindTrack<UMovieSceneBoolTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneBoolTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneBoolSection* Section         = Cast<UMovieSceneBoolSection>(Track->CreateNewSection());

		FFrameRate   TickResolution  = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame    = (InTime * TickResolution).FloorToFrame();

		Section->SetRange(TRange<FFrameNumber>(CurrentFrame));

		Section->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();

		FMovieSceneBoolChannel* BoolChannel = Section->GetChannelProxy().GetChannels<FMovieSceneBoolChannel>()[0];
		BoolChannel->SetDefault(PreviousValue);
		BoolChannel->GetData().AddKey(CurrentFrame, PreviousValue);

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieScenePropertyRecorder<bool>::AddKeyToSection(UMovieSceneSection* InSection, const FPropertyKey<bool>& InKey)
{
	InSection->GetChannelProxy().GetChannel<FMovieSceneBoolChannel>(0)->GetData().AddKey(InKey.Time, InKey.Value);
}

template <>
void FMovieScenePropertyRecorder<bool>::ReduceKeys(UMovieSceneSection* InSection)
{
}

template <>
bool FMovieScenePropertyRecorder<uint8>::ShouldAddNewKey(const uint8& InNewValue) const
{
	return InNewValue != PreviousValue;
}

template <>
UMovieSceneSection* FMovieScenePropertyRecorder<uint8>::AddSection(UObject* InObjectToRecord, UMovieScene* InMovieScene, const FGuid& InGuid, float InTime)
{
	if (!InObjectToRecord)
	{
		return nullptr;
	}

	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneByteTrack* Track = InMovieScene->FindTrack<UMovieSceneByteTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneByteTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneByteSection* Section = Cast<UMovieSceneByteSection>(Track->CreateNewSection());

		FFrameRate   TickResolution  = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame    = (InTime * TickResolution).FloorToFrame();

		Section->SetRange(TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame));

		Section->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();

		FMovieSceneByteChannel* Channel = Section->GetChannelProxy().GetChannel<FMovieSceneByteChannel>(0);
		Channel->SetDefault(PreviousValue);
		Channel->GetData().AddKey(CurrentFrame, PreviousValue);

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieScenePropertyRecorder<uint8>::AddKeyToSection(UMovieSceneSection* InSection, const FPropertyKey<uint8>& InKey)
{
	InSection->GetChannelProxy().GetChannel<FMovieSceneByteChannel>(0)->GetData().AddKey(InKey.Time, InKey.Value);
}

template <>
void FMovieScenePropertyRecorder<uint8>::ReduceKeys(UMovieSceneSection* InSection)
{
}

bool FMovieScenePropertyRecorderEnum::ShouldAddNewKey(const int64& InNewValue) const
{
	return InNewValue != PreviousValue;
}

UMovieSceneSection* FMovieScenePropertyRecorderEnum::AddSection(UObject* InObjectToRecord, UMovieScene* InMovieScene, const FGuid& InGuid, float InTime)
{
	if (!InObjectToRecord)
	{
		return nullptr;
	}

	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneEnumTrack* Track = InMovieScene->FindTrack<UMovieSceneEnumTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneEnumTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneEnumSection* Section = Cast<UMovieSceneEnumSection>(Track->CreateNewSection());

		FFrameRate   TickResolution  = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame    = (InTime * TickResolution).FloorToFrame();

		Section->SetRange(TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame));

		Section->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();

		FMovieSceneByteChannel* Channel = Section->GetChannelProxy().GetChannel<FMovieSceneByteChannel>(0);
		Channel->SetDefault(PreviousValue);
		Channel->GetData().AddKey(CurrentFrame, PreviousValue);

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

void FMovieScenePropertyRecorderEnum::AddKeyToSection(UMovieSceneSection* InSection, const FPropertyKey<int64>& InKey)
{
	InSection->GetChannelProxy().GetChannel<FMovieSceneByteChannel>(0)->GetData().AddKey(InKey.Time, InKey.Value);
}

void FMovieScenePropertyRecorderEnum::ReduceKeys(UMovieSceneSection* InSection)
{
}

template <>
bool FMovieScenePropertyRecorder<float>::ShouldAddNewKey(const float& InNewValue) const
{
	return true;
}

template <>
UMovieSceneSection* FMovieScenePropertyRecorder<float>::AddSection(UObject* InObjectToRecord, UMovieScene* InMovieScene, const FGuid& InGuid, float InTime)
{
	if (!InObjectToRecord)
	{
		return nullptr;
	}

	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneFloatTrack* Track = InMovieScene->FindTrack<UMovieSceneFloatTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneFloatTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneFloatSection* Section = Cast<UMovieSceneFloatSection>(Track->CreateNewSection());

		FFrameRate   TickResolution  = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame    = (InTime * TickResolution).FloorToFrame();

		Section->SetRange(TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame));

		Section->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();

		FMovieSceneFloatChannel* Channel = Section->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(0);
		check(Channel);
		Channel->SetDefault(PreviousValue);
		Channel->AddCubicKey(CurrentFrame, PreviousValue, RCTM_Break);

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieScenePropertyRecorder<float>::AddKeyToSection(UMovieSceneSection* InSection, const FPropertyKey<float>& InKey)
{
	InSection->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(0)->AddCubicKey(InKey.Time, InKey.Value);
}

template <>
void FMovieScenePropertyRecorder<float>::ReduceKeys(UMovieSceneSection* InSection)
{
	FKeyDataOptimizationParams Params;
	UE::MovieScene::Optimize(InSection->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(0), Params);
}

template <>
bool FMovieScenePropertyRecorder<FColor>::ShouldAddNewKey(const FColor& InNewValue) const
{
	return true;
}

template <>
UMovieSceneSection* FMovieScenePropertyRecorder<FColor>::AddSection(UObject* InObjectToRecord, UMovieScene* InMovieScene, const FGuid& InGuid, float InTime)
{
	if (!InObjectToRecord)
	{
		return nullptr;
	}

	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneColorTrack* Track = InMovieScene->FindTrack<UMovieSceneColorTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneColorTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneColorSection* Section = Cast<UMovieSceneColorSection>(Track->CreateNewSection());

		FFrameRate   TickResolution  = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame    = (InTime * TickResolution).FloorToFrame();

		Section->SetRange(TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame));

		Section->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();

		TArrayView<FMovieSceneFloatChannel*> FloatChannels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
		const float InvColor = 1.0f / 255.0f;

		FloatChannels[0]->SetDefault(PreviousValue.R);
		FloatChannels[0]->AddCubicKey(CurrentFrame, PreviousValue.R * InvColor, RCTM_Break);

		FloatChannels[1]->SetDefault(PreviousValue.G);
		FloatChannels[1]->AddCubicKey(CurrentFrame, PreviousValue.G * InvColor, RCTM_Break);

		FloatChannels[2]->SetDefault(PreviousValue.B);
		FloatChannels[2]->AddCubicKey(CurrentFrame, PreviousValue.B * InvColor, RCTM_Break);

		FloatChannels[3]->SetDefault(PreviousValue.A);
		FloatChannels[3]->AddCubicKey(CurrentFrame, PreviousValue.A * InvColor, RCTM_Break);

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieScenePropertyRecorder<FColor>::AddKeyToSection(UMovieSceneSection* InSection, const FPropertyKey<FColor>& InKey)
{
	static const float InvColor = 1.0f / 255.0f;
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	FloatChannels[0]->AddCubicKey(InKey.Time, InKey.Value.R * InvColor, RCTM_Break);
	FloatChannels[1]->AddCubicKey(InKey.Time, InKey.Value.G * InvColor, RCTM_Break);
	FloatChannels[2]->AddCubicKey(InKey.Time, InKey.Value.B * InvColor, RCTM_Break);
	FloatChannels[3]->AddCubicKey(InKey.Time, InKey.Value.A * InvColor, RCTM_Break);
}

template <>
void FMovieScenePropertyRecorder<FColor>::ReduceKeys(UMovieSceneSection* InSection)
{
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

	FKeyDataOptimizationParams Params;
	UE::MovieScene::Optimize(FloatChannels[0], Params);
	UE::MovieScene::Optimize(FloatChannels[1], Params);
	UE::MovieScene::Optimize(FloatChannels[2], Params);
	UE::MovieScene::Optimize(FloatChannels[3], Params);
}

template <>
bool FMovieScenePropertyRecorder<FVector3f>::ShouldAddNewKey(const FVector3f& InNewValue) const
{
	return true;
}

template <>
UMovieSceneSection* FMovieScenePropertyRecorder<FVector3f>::AddSection(UObject* InObjectToRecord, UMovieScene* InMovieScene, const FGuid& InGuid, float InTime)
{
	if (!InObjectToRecord)
	{
		return nullptr;
	}

	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneFloatVectorTrack* Track = InMovieScene->FindTrack<UMovieSceneFloatVectorTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneFloatVectorTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetNumChannelsUsed(3);
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneFloatVectorSection* Section = Cast<UMovieSceneFloatVectorSection>(Track->CreateNewSection());

		FFrameRate   TickResolution  = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame    = (InTime * TickResolution).FloorToFrame();

		Section->SetRange(TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame));

		Section->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();

		TArrayView<FMovieSceneFloatChannel*> FloatChannels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

		FloatChannels[0]->SetDefault(PreviousValue.X);
		FloatChannels[0]->AddCubicKey(CurrentFrame, PreviousValue.X, RCTM_Break);

		FloatChannels[1]->SetDefault(PreviousValue.Y);
		FloatChannels[1]->AddCubicKey(CurrentFrame, PreviousValue.Y, RCTM_Break);

		FloatChannels[2]->SetDefault(PreviousValue.Z);
		FloatChannels[2]->AddCubicKey(CurrentFrame, PreviousValue.Z, RCTM_Break);

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieScenePropertyRecorder<FVector3f>::AddKeyToSection(UMovieSceneSection* InSection, const FPropertyKey<FVector3f>& InKey)
{
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	FloatChannels[0]->AddCubicKey(InKey.Time, InKey.Value.X);
	FloatChannels[1]->AddCubicKey(InKey.Time, InKey.Value.Y);
	FloatChannels[2]->AddCubicKey(InKey.Time, InKey.Value.Z);
}

template <>
void FMovieScenePropertyRecorder<FVector3f>::ReduceKeys(UMovieSceneSection* InSection)
{
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

	FKeyDataOptimizationParams Params;
	UE::MovieScene::Optimize(FloatChannels[0], Params);
	UE::MovieScene::Optimize(FloatChannels[1], Params);
	UE::MovieScene::Optimize(FloatChannels[2], Params);
}

template <>
bool FMovieScenePropertyRecorder<FVector3d>::ShouldAddNewKey(const FVector3d& InNewValue) const
{
	return true;
}

template <>
UMovieSceneSection* FMovieScenePropertyRecorder<FVector3d>::AddSection(UObject* InObjectToRecord, UMovieScene* InMovieScene, const FGuid& InGuid, float InTime)
{
	if (!InObjectToRecord)
	{
		return nullptr;
	}

	FName TrackName = *Binding.GetPropertyPath();
	UMovieSceneDoubleVectorTrack* Track = InMovieScene->FindTrack<UMovieSceneDoubleVectorTrack>(InGuid, TrackName);
	if (!Track)
	{
		Track = InMovieScene->AddTrack<UMovieSceneDoubleVectorTrack>(InGuid);
	}
	else
	{
		Track->RemoveAllAnimationData();
	}

	if (Track)
	{
		Track->SetNumChannelsUsed(3);
		Track->SetPropertyNameAndPath(Binding.GetPropertyName(), Binding.GetPropertyPath());

		UMovieSceneDoubleVectorSection* Section = Cast<UMovieSceneDoubleVectorSection>(Track->CreateNewSection());

		FFrameRate   TickResolution  = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame    = (InTime * TickResolution).FloorToFrame();

		Section->SetRange(TRange<FFrameNumber>::Inclusive(CurrentFrame, CurrentFrame));

		Section->TimecodeSource = SequenceRecorderUtils::GetTimecodeSource();

		TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = Section->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

		DoubleChannels[0]->SetDefault(PreviousValue.X);
		DoubleChannels[0]->AddCubicKey(CurrentFrame, PreviousValue.X, RCTM_Break);

		DoubleChannels[1]->SetDefault(PreviousValue.Y);
		DoubleChannels[1]->AddCubicKey(CurrentFrame, PreviousValue.Y, RCTM_Break);

		DoubleChannels[2]->SetDefault(PreviousValue.Z);
		DoubleChannels[2]->AddCubicKey(CurrentFrame, PreviousValue.Z, RCTM_Break);

		Track->AddSection(*Section);

		return Section;
	}

	return nullptr;
}

template <>
void FMovieScenePropertyRecorder<FVector3d>::AddKeyToSection(UMovieSceneSection* InSection, const FPropertyKey<FVector3d>& InKey)
{
	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	DoubleChannels[0]->AddCubicKey(InKey.Time, InKey.Value.X);
	DoubleChannels[1]->AddCubicKey(InKey.Time, InKey.Value.Y);
	DoubleChannels[2]->AddCubicKey(InKey.Time, InKey.Value.Z);
}

template <>
void FMovieScenePropertyRecorder<FVector3d>::ReduceKeys(UMovieSceneSection* InSection)
{
	TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

	FKeyDataOptimizationParams Params;
	UE::MovieScene::Optimize(DoubleChannels[0], Params);
	UE::MovieScene::Optimize(DoubleChannels[1], Params);
	UE::MovieScene::Optimize(DoubleChannels[2], Params);
}

template class FMovieScenePropertyRecorder<bool>;
template class FMovieScenePropertyRecorder<uint8>;
template class FMovieScenePropertyRecorder<float>;
template class FMovieScenePropertyRecorder<FColor>;
template class FMovieScenePropertyRecorder<FVector3f>;
template class FMovieScenePropertyRecorder<FVector3d>;
