// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceRecorder.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/WorldSettings.h"
#include "ISequenceAudioRecorder.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Engine/Texture2D.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "AssetRegistry/AssetData.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor.h"
#include "EngineGlobals.h"

#include "LevelEditor.h"
#include "AnimationRecorder.h"
#include "ActorRecording.h"
#include "SequenceRecordingBase.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "SequenceRecorderUtils.h"
#include "SequenceRecorderSettings.h"
#include "ObjectTools.h"
#include "Features/IModularFeatures.h"
#include "Engine/DemoNetDriver.h"
#include "Engine/Selection.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "LevelSequenceActor.h"
#include "IAssetViewport.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Sections/MovieSceneAudioSection.h"
#include "Sound/SoundWave.h"
#include "SequenceRecorderActorGroup.h"
#include "MovieSceneTimeHelpers.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Camera/CameraActor.h"
#include "ISequenceRecorderExtender.h"
#include "ScopedTransaction.h"
#include "Features/IModularFeatures.h"
#include "ScopedTransaction.h"
#include "LevelSequencePlayer.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "SequenceRecorder"

FSequenceRecorder::FSequenceRecorder()
	: bQueuedRecordingsDirty(false)
	, bWasImmersive(false)
	, CurrentDelay(0.0f)
	, CurrentTime(0.0f)
{
}

void FSequenceRecorder::Initialize()
{
	// load textures we use for the countdown/recording display
	CountdownTexture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EditorResources/SequenceRecorder/Countdown.Countdown"), nullptr, LOAD_None, nullptr);
	CountdownTexture->AddToRoot();
	RecordingIndicatorTexture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EditorResources/SequenceRecorder/RecordingIndicator.RecordingIndicator"), nullptr, LOAD_None, nullptr);
	RecordingIndicatorTexture->AddToRoot();

	// register built-in recorders
	IModularFeatures::Get().RegisterModularFeature("MovieSceneSectionRecorderFactory", &AnimationSectionRecorderFactory);
	IModularFeatures::Get().RegisterModularFeature("MovieSceneSectionRecorderFactory", &TransformSectionRecorderFactory);
	IModularFeatures::Get().RegisterModularFeature("MovieSceneSectionRecorderFactory", &MultiPropertySectionRecorder);

	RefreshNextSequence();
}

void FSequenceRecorder::Shutdown()
{
	// unregister built-in recorders
	IModularFeatures::Get().UnregisterModularFeature("MovieSceneSectionRecorderFactory", &AnimationSectionRecorderFactory);
	IModularFeatures::Get().UnregisterModularFeature("MovieSceneSectionRecorderFactory", &TransformSectionRecorderFactory);
	IModularFeatures::Get().UnregisterModularFeature("MovieSceneSectionRecorderFactory", &MultiPropertySectionRecorder);

	if(CountdownTexture.IsValid())
	{
		CountdownTexture->RemoveFromRoot();
		CountdownTexture.Reset();
	}
	if(RecordingIndicatorTexture.IsValid())
	{
		RecordingIndicatorTexture->RemoveFromRoot();
		RecordingIndicatorTexture.Reset();
	}
}

FSequenceRecorder& FSequenceRecorder::Get()
{
	static FSequenceRecorder SequenceRecorder;
	return SequenceRecorder;
}


bool FSequenceRecorder::IsRecordingQueued(AActor* Actor) const
{
	for (UActorRecording* QueuedRecording : QueuedActorRecordings)
	{
		if (QueuedRecording->GetActorToRecord() == Actor)
		{
			return true;
		}
	}

	return false;
}

bool FSequenceRecorder::IsRecordingQueued(UObject* SequenceRecordingObjectToRecord) const
{
	for (USequenceRecordingBase* QueuedRecording : QueuedRecordings)
	{
		if (QueuedRecording->GetObjectToRecord() == SequenceRecordingObjectToRecord)
		{
			return true;
		}
	}

	return false;
}

UActorRecording* FSequenceRecorder::FindRecording(AActor* Actor) const
{
	for (UActorRecording* QueuedRecording : QueuedActorRecordings)
	{
		if (QueuedRecording->GetActorToRecord() == Actor)
		{
			return QueuedRecording;
		}
	}

	return nullptr;
}

USequenceRecordingBase* FSequenceRecorder::FindRecording(UObject* SequenceRecordingObjectToRecord) const
{
	for (USequenceRecordingBase* QueuedRecording : QueuedRecordings)
	{
		if (QueuedRecording->GetObjectToRecord() == SequenceRecordingObjectToRecord)
		{
			return QueuedRecording;
		}
	}

	return nullptr;
}

void FSequenceRecorder::StartAllQueuedRecordings()
{
	for (USequenceRecordingBase* QueuedRecording : QueuedRecordings)
	{
		QueuedRecording->StartRecording(CurrentSequence.Get(), CurrentTime, PathToRecordTo, SequenceName);
	}
}

void FSequenceRecorder::StopAllQueuedRecordings()
{
	for (USequenceRecordingBase* QueuedRecording : QueuedRecordings)
	{
		QueuedRecording->StopRecording(CurrentSequence.Get(), CurrentTime);
	}
}

void FSequenceRecorder::AddNewQueuedRecordingsForSelectedActors()
{
	bool bAnySelectedActorsAdded = false;
	TArray<AActor*> EntireSelection;

	GEditor->GetSelectedActors()->GetSelectedObjects(EntireSelection);

	TArray<AActor*> ActorsToRecord;
	for (AActor* Actor : EntireSelection)
	{
		if (!FindRecording(Actor))
		{
			if (UActorRecording* NewRecording = AddNewQueuedRecording(Actor))
			{
				bAnySelectedActorsAdded = true;
			}
		}
	}

	if (!bAnySelectedActorsAdded)
	{
		AddNewQueuedRecording();
	}
}

/** Helper function - get the first PIE world (or first PIE client world if there is more than one) */
static UWorld* GetFirstPIEWorld()
{
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.World() != nullptr && Context.World()->IsPlayInEditor())
		{
			if(Context.World()->GetNetMode() == ENetMode::NM_Standalone ||
				(Context.World()->GetNetMode() == ENetMode::NM_Client && Context.PIEInstance == 2))
			{
				return Context.World();
			}
		}
	}

	return nullptr;
}

void FSequenceRecorder::AddNewQueuedRecordingForCurrentPlayer()
{
	if (UWorld* PIEWorld = GetFirstPIEWorld())
	{
		APlayerController* Controller = GEngine->GetFirstLocalPlayerController(PIEWorld);
		if(Controller && Controller->GetPawn())
		{
			APawn* CurrentPlayer = Controller->GetPawn();
			if (!FindRecording(CurrentPlayer))
			{
				AddNewQueuedRecording(CurrentPlayer);
			}
		}
	}
}

bool FSequenceRecorder::CanAddNewQueuedRecordingForCurrentPlayer() const
{
	if (UWorld* PIEWorld = GetFirstPIEWorld())
	{
		APlayerController* Controller = GEngine->GetFirstLocalPlayerController(PIEWorld);
		if(Controller && Controller->GetPawn())
		{
			APawn* CurrentPlayer = Controller->GetPawn();
			if (!FindRecording(CurrentPlayer))
			{
				return true;
			}
		}
	}
	return false;
}

UActorRecording* FSequenceRecorder::AddNewQueuedRecording(AActor* Actor, UAnimSequence* AnimSequence, float Length)
{
	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

	UActorRecording* ActorRecording = NewObject<UActorRecording>(CurrentRecorderGroup.IsValid() ? CurrentRecorderGroup.Get() : (UObject*)GetTransientPackage());
	ActorRecording->AddToRoot();
	ActorRecording->SetActorToRecord(Actor);
	ActorRecording->TargetAnimation = AnimSequence;
	ActorRecording->AnimationSettings.Length = Length;

	TWeakObjectPtr<USequenceRecorderActorGroup> RecordingGroup = GetCurrentRecordingGroup();
	if (RecordingGroup.IsValid())
	{
		ActorRecording->bCreateLevelSequence = RecordingGroup.Get()->bSpecifyTargetLevelSequence;
	}

	// We always record in world space as we need animations to record root motion
	ActorRecording->AnimationSettings.bRecordInWorldSpace = true;

	UMovieScene3DTransformSectionRecorderSettings* TransformSettings = ActorRecording->ActorSettings.GetSettingsObject<UMovieScene3DTransformSectionRecorderSettings>();
	check(TransformSettings);
	TransformSettings->bRecordTransforms = true;

	// auto-save assets in non-editor runtime
	if(GEditor == nullptr)
	{
		ActorRecording->AnimationSettings.bAutoSaveAsset = true;
	}

	QueuedRecordings.Add(ActorRecording);
	QueuedActorRecordings.Add(ActorRecording);
	if (CurrentRecorderGroup.IsValid() && !CurrentRecorderGroup->RecordedActors.Contains(ActorRecording))
	{
		CurrentRecorderGroup->RecordedActors.Add(ActorRecording);
	}

	bQueuedRecordingsDirty = true;

	return ActorRecording;
}

USequenceRecordingBase* FSequenceRecorder::AddNewQueuedRecording(UObject* SequenceRecordingObjectToRecord)
{
	for (TSharedPtr<ISequenceRecorderExtender> RecorderExtender : SequenceRecorderExtenders)
	{
		USequenceRecordingBase* RecordingBase = RecorderExtender->AddNewQueueRecording(SequenceRecordingObjectToRecord);
		if (RecordingBase)
		{
			QueuedRecordings.Add(RecordingBase);
			bQueuedRecordingsDirty = true;
			return RecordingBase;
		}
	}

	if (AActor* Actor = Cast<AActor>(SequenceRecordingObjectToRecord))
	{
		return AddNewQueuedRecording(Actor);
	}

	return nullptr;
}

void FSequenceRecorder::RemoveQueuedRecording(class USequenceRecordingBase* Recording)
{
	if (QueuedRecordings.RemoveSwap(Recording) > 0)
	{
		Recording->RemoveFromRoot();
		BuildQueuedRecordings();

		bQueuedRecordingsDirty = true;
	}
}

void FSequenceRecorder::ClearQueuedRecordings()
{
	if(IsRecording())
	{
		UE_LOGF(LogAnimation, Display, "Couldnt clear queued recordings while recording is in progress");
	}
	else
	{
		for (USequenceRecordingBase* QueuedRecording : QueuedRecordings)
		{
			QueuedRecording->RemoveFromRoot();
		}
		QueuedRecordings.Empty();
		QueuedActorRecordings.Empty();
		for (TSharedPtr<ISequenceRecorderExtender> RecorderExtender : SequenceRecorderExtenders)
		{
			RecorderExtender->BuildQueuedRecordings(QueuedRecordings);
		}

		bQueuedRecordingsDirty = true;
	}
}

bool FSequenceRecorder::HasQueuedRecordings() const
{
	return QueuedRecordings.Num() > 0;
}

bool FSequenceRecorder::IsRecording() const
{
	for(USequenceRecordingBase* Recording : QueuedRecordings)
	{
		if (Recording->IsRecording())
		{
			return true;
		}
	}
	return false;
}


void FSequenceRecorder::Tick(float DeltaSeconds)
{
	const float FirstFrameTickLimit = 1.0f / 30.0f;

	// in-editor we can get a long frame update because of the searching we need to do to filter actors
	if(DeltaSeconds > FirstFrameTickLimit && CurrentTime < FirstFrameTickLimit * 2.0f && IsRecording())
	{
		DeltaSeconds = FirstFrameTickLimit;
	}

	// if a replay recording is in progress and channels are paused, wait until we have data again before recording
	if (UDemoNetDriver* DemoNetDriver = CurrentReplayWorld.IsValid() ? CurrentReplayWorld->GetDemoNetDriver() : nullptr)
	{
		if (DemoNetDriver->GetChannelsArePaused())
		{
			return;
		}
	}

	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

	// Sequence Recorder supports modifying the global time dilation when a recording is started. This can be useful to easily capture a scene in slow
	// motion and it will record the resulting data at the slowed down speed. Recording the data at the slowed down speed is not always desirable - an 
	// example is playing back the scene in slow motion to make it easier to focus on fast-paced action but wanting the resulting level sequence to be
	// recorded at full speed. To accomplish this we can scale the delta time by the time dilation to counteract the effect on the recorded data.
	float ScaledDeltaSeconds = DeltaSeconds;
	if (Settings->bIgnoreTimeDilation && CurrentRecordingWorld.IsValid())
	{
		AWorldSettings* WorldSettings = CurrentRecordingWorld->GetWorldSettings();
		if (WorldSettings)
		{
			// We retrieve the time dilation from the world every frame in case the game is modifying time dilation as we play.
			ScaledDeltaSeconds = DeltaSeconds * WorldSettings->TimeDilation;
		}
	}
		
	// Animation Recorder automatically increments its internal frame it's recording to based on incrementing by delta time so modifying delta time keeps
	// the animation recorder in sync with our time dilation options.
	FAnimationRecorderManager::Get().Tick(ScaledDeltaSeconds);
	for(USequenceRecordingBase* Recording : QueuedRecordings)
	{
		// Actor Recordings take a specific time to record to, so we only increment CurrentTime by the scaled delta-time.
		Recording->Tick(CurrentSequence.Get(), CurrentTime);
	}

	if(CurrentDelay > 0.0f)
	{
		CurrentDelay -= DeltaSeconds;
		if(CurrentDelay <= 0.0f)
		{
			CurrentDelay = 0.0f;
			StartRecordingInternal(nullptr);

			if (!IsRecording())
			{
				RestoreImmersive();
			}
		}
	}

	if (IsRecording())
	{
		// By increasing CurrentTime by delta time, this causes the UI and auto-shutoff to respect the time dilation settings as well.
		CurrentTime += ScaledDeltaSeconds;

		// check if all our actor recordings are finished or we timed out
		if(QueuedRecordings.Num() > 0)
		{
			bool bAllFinished = true;
			for(USequenceRecordingBase* Recording : QueuedRecordings)
			{
				if(Recording->IsRecording())
				{
					bAllFinished = false;
					break;
				}
			}

			bool bWaitingForTargetLevelSequenceLength = false;
			TWeakObjectPtr<USequenceRecorderActorGroup> RecordingGroup = GetCurrentRecordingGroup();
			if (RecordingGroup.IsValid() && RecordingGroup.Get()->bRecordTargetLevelSequenceLength)
			{
				if (CurrentSequence.IsValid())
				{
					UMovieScene* CurrentMovieScene = CurrentSequence.Get()->GetMovieScene();
					if (CurrentMovieScene && !CurrentMovieScene->GetPlaybackRange().IsEmpty())
					{
						bWaitingForTargetLevelSequenceLength = true;

						float SequenceDurationInSeconds = FFrameNumber(UE::MovieScene::DiscreteSize(CurrentMovieScene->GetPlaybackRange())) / CurrentMovieScene->GetTickResolution();
						if (CurrentTime >= SequenceDurationInSeconds)
						{
							StopRecording(Settings->bAllowLooping);
						}
					}
				}
			}

			if(bAllFinished || (Settings->SequenceLength > 0.0f && CurrentTime >= Settings->SequenceLength && !bWaitingForTargetLevelSequenceLength))
			{
				StopRecording(Settings->bAllowLooping);
			}
		}

		auto RemoveDeadObjectPredicate = 
			[&](USequenceRecordingBase* Recording)
			{
				if(!Recording->GetObjectToRecord())
				{
					DeadRecordings.Add(Recording);
					return true;
				}
				return false; 
			};

		// remove all dead actors
		int32 Removed = QueuedRecordings.RemoveAll(RemoveDeadObjectPredicate);
		if(Removed > 0)
		{
			BuildQueuedRecordings();
			bQueuedRecordingsDirty = true;
		}
	}
}

void FSequenceRecorder::DrawDebug(UCanvas* InCanvas, APlayerController* InPlayerController)
{
	const float NumFrames = 9.0f;
	const bool bCountingDown = CurrentDelay > 0.0f && CurrentDelay < NumFrames;

	if(bCountingDown)
	{
		const FVector2D IconSize(128.0f, 128.0f);
		const FVector2f HalfIconSize(64.0f, 64.0f);
		const float LineThickness = 2.0f;

		FVector2f Center;
		InCanvas->GetCenter(Center.X, Center.Y);
		FVector2f IconPosition = Center - HalfIconSize;

		InCanvas->SetDrawColor(FColor::White);

		FCanvasIcon Icon = UCanvas::MakeIcon(CountdownTexture.Get(), FMath::FloorToFloat(NumFrames - CurrentDelay) * IconSize.X, 0.0f, IconSize.X, IconSize.Y);
		InCanvas->DrawIcon(Icon, IconPosition.X, IconPosition.Y);
		
		// draw 'clock' line
		const float Angle = 2.0f * PI * FMath::Fmod(CurrentDelay, 1.0f);
		const FVector2D AxisX(0.f, -1.f);
		const FVector2D AxisY(-1.f, 0.f);
		const FVector2D EndPos = FVector2D(Center) + (AxisX * FMath::Cos(Angle) + AxisY * FMath::Sin(Angle)) * (InCanvas->SizeX + InCanvas->SizeY);
		FCanvasLineItem LineItem(FVector2D(Center), EndPos);
		LineItem.LineThickness = LineThickness;
		LineItem.SetColor(FLinearColor::Black);
		InCanvas->DrawItem(LineItem);

		// draw 'crosshairs'
		LineItem.Origin = FVector(0.0f, Center.Y, 0.0f);
		LineItem.EndPos = FVector(InCanvas->SizeX, Center.Y, 0.0f);
		InCanvas->DrawItem(LineItem);

		LineItem.Origin = FVector(Center.X, 0.0f, 0.0f);
		LineItem.EndPos = FVector(Center.X, InCanvas->SizeY, 0.0f);
		InCanvas->DrawItem(LineItem);
	}

	if(bCountingDown || IsRecording())
	{
		const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();
	
		FText LabelText;
		if (Settings->bCreateLevelSequence)
		{
			if (CurrentSequence.IsValid())
			{
				LabelText = FText::Format(LOCTEXT("RecordingIndicatorFormat", "{0}"), FText::FromName(CurrentSequence.Get()->GetFName()));
			}
			else
			{
				LabelText = FText::Format(LOCTEXT("RecordingIndicatorPending", "Pending recording: {0}"), FText::FromString(NextSequenceName));
			}
		}


		float TimeAccumulator = CurrentTime;
		float Hours = FMath::FloorToFloat(TimeAccumulator / (60.0f * 60.0f));
		TimeAccumulator -= Hours * 60.0f * 60.0f;
		float Minutes = FMath::FloorToFloat(TimeAccumulator / 60.0f);
		TimeAccumulator -= Minutes * 60.0f;
		float Seconds = FMath::FloorToFloat(TimeAccumulator);
		TimeAccumulator -= Seconds;
		float Frames = FMath::FloorToFloat(GetDefault<USequenceRecorderSettings>()->DefaultAnimationSettings.SampleFrameRate.AsFrameTime(TimeAccumulator).AsDecimal());

		FNumberFormattingOptions Options;
		Options.MinimumIntegralDigits = 2;
		Options.MaximumIntegralDigits = 2;

		FFormatNamedArguments NamedArgs;
		NamedArgs.Add(TEXT("Hours"), FText::AsNumber((int32)Hours, &Options));
		NamedArgs.Add(TEXT("Minutes"), FText::AsNumber((int32)Minutes, &Options));
		NamedArgs.Add(TEXT("Seconds"), FText::AsNumber((int32)Seconds, &Options));
		NamedArgs.Add(TEXT("Frames"), FText::AsNumber((int32)Frames, &Options));
		FText TimeText = FText::Format(LOCTEXT("RecordingTimerFormat", "{Hours}:{Minutes}:{Seconds}:{Frames}"), NamedArgs);

		const FVector2D IconSize(32.0f, 32.0f);
		const FVector2D Offset(8.0f, 32.0f);

		InCanvas->SetDrawColor(FColor::White);

		FVector2D IconPosition(Offset.X, InCanvas->SizeY - (Offset.Y + IconSize.Y));
		FCanvasIcon Icon = UCanvas::MakeIcon(RecordingIndicatorTexture.Get(), FMath::FloorToFloat(NumFrames - CurrentDelay) * IconSize.X, 0.0f, IconSize.X, IconSize.Y);
		InCanvas->DrawIcon(Icon, IconPosition.X, IconPosition.Y);

		const float TextScale = 1.2f;
		float TextPositionY = 0.0f;
		// draw label
		{
			float XSize, YSize;
			InCanvas->TextSize(GEngine->GetLargeFont(), LabelText.ToString(), XSize, YSize, TextScale, TextScale);

			TextPositionY = (IconPosition.Y + (IconSize.Y * 0.5f)) - (YSize * 0.5f);

			FFontRenderInfo Info;
			Info.bEnableShadow = true;
			InCanvas->DrawText(GEngine->GetLargeFont(), LabelText, IconPosition.X + IconSize.X + 4.0f, TextPositionY, TextScale, TextScale, Info);
		}
		// draw time
		{
			float XSize, YSize;
			InCanvas->TextSize(GEngine->GetLargeFont(), TimeText.ToString(), XSize, YSize, TextScale, TextScale);

			FVector2D TimePosition(InCanvas->SizeX - (Offset.X + XSize), TextPositionY);

			FFontRenderInfo Info;
			Info.bEnableShadow = true;
			InCanvas->DrawText(GEngine->GetLargeFont(), TimeText, TimePosition.X, TimePosition.Y, TextScale, TextScale, Info);
		}
	}
}

bool FSequenceRecorder::StartRecording(const FString& InPathToRecordTo, const FString& InSequenceName)
{
	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

	if(InPathToRecordTo.Len() > 0)
	{
		PathToRecordTo = InPathToRecordTo;
	}
	else
	{
		PathToRecordTo = GetSequenceRecordingBasePath();
	}

	if(InSequenceName.Len() > 0)
	{
		SequenceName = InSequenceName;
	}
	else
	{
		SequenceName = GetSequenceRecordingName().Len() > 0 ? GetSequenceRecordingName() : TEXT("RecordedSequence");
	}

	PathToRecordTo /= SequenceName;

	SetImmersive();

	RefreshNextSequence();

	if(Settings->RecordingDelay > 0.0f)
	{
		CurrentDelay = Settings->RecordingDelay;

		UE_LOGF(LogAnimation, Display, "Starting sequence recording with delay of %f seconds", CurrentDelay);

		return QueuedRecordings.Num() > 0;
	}
	
	return StartRecordingInternal(nullptr);
}

bool FSequenceRecorder::StartRecordingForReplay(UWorld* World, const FSequenceRecorderActorFilter& ActorFilter)
{
	// set up our recording settings
	USequenceRecorderSettings* Settings = GetMutableDefault<USequenceRecorderSettings>();
	
	Settings->bCreateLevelSequence = true;
	Settings->SequenceLength = 0.0f;
	Settings->RecordingDelay = 0.0f;
	Settings->bRecordNearbySpawnedActors = true;
	Settings->NearbyActorRecordingProximity = 0.0f;
	Settings->bRecordWorldSettingsActor = true;
	Settings->ActorFilter = ActorFilter;

	CurrentReplayWorld = World;
	
	return StartRecordingInternal(World);
}

bool FSequenceRecorder::StartRecordingInternal(UWorld* World)
{
	CurrentTime = 0.0f;

	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

	UWorld* ActorWorld = nullptr;
	if(World != nullptr || (QueuedActorRecordings.Num() > 0 && QueuedActorRecordings[0]->GetActorToRecord() != nullptr))
	{
		ActorWorld = World != nullptr ? World : QueuedActorRecordings[0]->GetActorToRecord()->GetWorld();
	}

	CurrentRecordingWorld = ActorWorld;

	if(Settings->bRecordWorldSettingsActor)
	{
		if(ActorWorld)
		{
			AWorldSettings* WorldSettings = ActorWorld->GetWorldSettings();
			if (!IsRecordingQueued(WorldSettings))
			{
				UActorRecording* WorldSettingsRecording = AddNewQueuedRecording(WorldSettings);
				WorldSettingsRecording->bCreateLevelSequence = false;
			}
		}
	}

	// kick off level sequence actors we are syncing to
	for(TWeakObjectPtr<ALevelSequenceActor> LevelSequenceActor : Settings->LevelSequenceActorsToTrigger)
	{
		if(LevelSequenceActor.IsValid())
		{
			// find a counterpart in the PIE world if this actor is not
			ALevelSequenceActor* ActorToTrigger = LevelSequenceActor.Get();
			if(!ActorToTrigger->GetWorld()->IsPlayInEditor())
			{
				ActorToTrigger = Cast<ALevelSequenceActor>(EditorUtilities::GetSimWorldCounterpartActor(ActorToTrigger));
			}

			if(ActorToTrigger)
			{
				// Duplicate the level sequence we want to trigger so that we can playback the level sequence and record to it at the same time
				ALevelSequenceActor* DupActorToTrigger = ActorToTrigger->GetWorld()->SpawnActor<ALevelSequenceActor>();
				if (!DupActorToTrigger)
				{
					UE_LOGF(LogAnimation, Display, "Unabled to spawn actor to trigger: (%ls)", *ActorToTrigger->GetPathName());
					continue;
				}

				ULevelSequence* DupLevelSequence = CastChecked<ULevelSequence>(StaticDuplicateObject(ActorToTrigger->GetSequence(), ActorToTrigger->GetOuter(), NAME_None, RF_AllFlags & ~RF_Transactional));
				DupActorToTrigger->SetSequence(DupLevelSequence);

				// Always initialize the player so that the playback settings/range can be initialized from editor.
				DupActorToTrigger->InitializePlayer();
				
				ULevelSequencePlayer* SequencePlayer = DupActorToTrigger->GetSequencePlayer();
				if (SequencePlayer)
				{
					SequencePlayer->SetDisableCameraCuts(true);
					SequencePlayer->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(0, EUpdatePositionMethod::Jump));
					SequencePlayer->Play();
				}
				else
				{
					UE_LOGF(LogAnimation, Display, "Level sequence (%ls) is not initialized for playback", *ActorToTrigger->GetPathName());
				}

				DupActorsToTrigger.Add(DupActorToTrigger);
			}
		}
	}

	if(QueuedRecordings.Num() > 0)
	{
		ULevelSequence* LevelSequence = nullptr;
		
		if(Settings->bCreateLevelSequence)
		{
			TWeakObjectPtr<USequenceRecorderActorGroup> RecordingGroup = GetCurrentRecordingGroup();
			if (RecordingGroup.IsValid() && RecordingGroup.Get()->bSpecifyTargetLevelSequence && RecordingGroup.Get()->TargetLevelSequence != nullptr)
			{
				LevelSequence = RecordingGroup.Get()->TargetLevelSequence;

				if (RecordingGroup.Get()->bDuplicateTargetLevelSequence)
				{
					IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();

					FString NewSequenceName = SequenceRecorderUtils::MakeNewAssetName(PathToRecordTo, SequenceName);

					LevelSequence = CastChecked<ULevelSequence>(SequenceRecorderUtils::DuplicateAsset(PathToRecordTo, SequenceName, RecordingGroup.Get()->TargetLevelSequence));
					
					RecordingGroup.Get()->TargetLevelSequence = LevelSequence;
				}
			}
			else
			{
				LevelSequence = SequenceRecorderUtils::MakeNewAsset<ULevelSequence>(PathToRecordTo, SequenceName);

				if (LevelSequence)
				{
					LevelSequence->Initialize();
				}

				if (RecordingGroup.IsValid() && RecordingGroup.Get()->bSpecifyTargetLevelSequence)
				{
					RecordingGroup.Get()->TargetLevelSequence = LevelSequence;
				}
			}

			if(LevelSequence)
			{
				CurrentSequence = LevelSequence;

				FAssetRegistryModule::AssetCreated(LevelSequence);

				RefreshNextSequence();
			}
		}

		// register for spawning delegate in the world(s) of recorded actors
		for(UActorRecording* Recording : QueuedActorRecordings)
		{
			if(Recording->GetActorToRecord() != nullptr)
			{
				UWorld* ActorToRecordWorld = Recording->GetActorToRecord()->GetWorld();
				if(ActorToRecordWorld != nullptr)
				{
					FDelegateHandle* FoundHandle = ActorSpawningDelegateHandles.Find(ActorToRecordWorld);
					if(FoundHandle == nullptr)
					{
						FDelegateHandle NewHandle = ActorToRecordWorld->AddOnActorSpawnedHandler(FOnActorSpawned::FDelegate::CreateRaw(this, &FSequenceRecorder::HandleActorSpawned));
						ActorSpawningDelegateHandles.Add(ActorToRecordWorld, NewHandle);
					}
				}
			}
		}

		// start recording 
		bool bAnyRecordingsStarted = false;
		for(USequenceRecordingBase* Recording : QueuedRecordings)
		{
			if(Recording->StartRecording(CurrentSequence.Get(), CurrentTime, PathToRecordTo, SequenceName))
			{
				bAnyRecordingsStarted = true;
			}
		}

		if(!bAnyRecordingsStarted)
		{
			// if we couldnt start a recording, stop all others
			TArray<FAssetData> AssetsToCleanUp;
			if(LevelSequence)
			{
				AssetsToCleanUp.Add(LevelSequence);
			}

			for(USequenceRecordingBase* Recording : QueuedRecordings)
			{
				Recording->StopRecording(CurrentSequence.Get(), CurrentTime);
			}

			// clean up any assets that we can
			if(AssetsToCleanUp.Num() > 0)
			{
				ObjectTools::DeleteAssets(AssetsToCleanUp, false);
			}
		}

#if WITH_EDITOR
		// if recording via PIE, be sure to stop recording cleanly when PIE ends
		if (ActorWorld && ActorWorld->IsPlayInEditor())
		{
			FEditorDelegates::EndPIE.AddRaw(this, &FSequenceRecorder::HandleEndPIE);
		}
#endif

		if (LevelSequence)
		{
			UE_LOGF(LogAnimation, Display, "Started recording sequence %ls", *LevelSequence->GetPathName());
		}

		// If we created an audio recorder at the start of the count down, then start recording
		// Create the audio recorder now before the count down finishes
		if (Settings->RecordAudio != EAudioRecordingMode::None)
		{
			if (LevelSequence)
			{
				FDirectoryPath AudioDirectory;
				AudioDirectory.Path = PathToRecordTo;
				if (Settings->AudioSubDirectory.Len())
				{
					AudioDirectory.Path /= Settings->AudioSubDirectory;
				}

				ISequenceRecorder& Recorder = FModuleManager::Get().LoadModuleChecked<ISequenceRecorder>("SequenceRecorder");

				FSequenceAudioRecorderSettings AudioSettings;
				AudioSettings.Directory = AudioDirectory;
				AudioSettings.AssetName = SequenceRecorderUtils::MakeNewAssetName(AudioDirectory.Path, LevelSequence->GetName());
				AudioSettings.RecordingDuration = Settings->SequenceLength;
				AudioSettings.GainDb = Settings->AudioGain;
				AudioSettings.bSplitChannels = Settings->bSplitAudioChannelsIntoSeparateTracks;

				AudioRecorder = Recorder.CreateAudioRecorder();
				if (AudioRecorder)
				{
					AudioRecorder->Start(AudioSettings);
				}
			}
			else
			{
				UE_LOGF(LogAnimation, Display, "'Create Level Sequence' needs to be enabled for audio recording");
			}
		}

		// Cache the current global time dilation in case the user is already using some form of slow-mo when they start recording.
		if (CurrentRecordingWorld.IsValid())
		{
			AWorldSettings* WorldSettings = CurrentRecordingWorld->GetWorldSettings();
			if (WorldSettings)
			{
				CachedGlobalTimeDilation = WorldSettings->TimeDilation;
				WorldSettings->SetTimeDilation(Settings->GlobalTimeDilation);
			}
		}

		if (OnRecordingStartedDelegate.IsBound())
		{
			OnRecordingStartedDelegate.Broadcast(CurrentSequence.Get());
		}

		return true;
	}
	else
	{
		UE_LOGF(LogAnimation, Display, "No recordings queued, aborting recording");
	}

	return false;
} 

void FSequenceRecorder::HandleEndPIE(bool bSimulating)
{
	StopRecording();

#if WITH_EDITOR
	FEditorDelegates::EndPIE.RemoveAll(this);
#endif
}

bool FSequenceRecorder::StopRecording(bool bAllowLooping)
{
	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

	RestoreImmersive();

	if (!IsRecording())
	{
		CurrentDelay = 0.0f;
	
		return false;
	}

	FScopedTransaction ScopeTransaction(LOCTEXT("ProcessedRecording", "Processed Recording"));

	// 1 step for the audio processing
	static const uint8 NumAdditionalSteps = 1;

	FScopedSlowTask SlowTask((float)(QueuedRecordings.Num() + DeadRecordings.Num() + NumAdditionalSteps), LOCTEXT("ProcessingRecording", "Processing Recording"));
	SlowTask.MakeDialog(false, true);

	// Process audio first so it doesn't record while we're processing the other captured state
	ULevelSequence* LevelSequence = CurrentSequence.Get();

	SlowTask.EnterProgressFrame(1.f, LOCTEXT("ProcessingAudio", "Processing Audio"));
	if (AudioRecorder && LevelSequence)
	{
		TArray<USoundWave*> RecordedSoundWaves;
		AudioRecorder->Stop(RecordedSoundWaves);
		AudioRecorder.Reset();

		if (RecordedSoundWaves.Num())
		{
			// Add a new audio track to the level sequence		
			UMovieScene* MovieScene = LevelSequence->GetMovieScene();
			UMovieSceneAudioTrack* RecordedAudioTrack = nullptr;

			FText RecordedAudioTrackName = Settings->AudioTrackName;
			for (auto Track : MovieScene->GetTracks())
			{
				if (Track->IsA(UMovieSceneAudioTrack::StaticClass()) && Track->GetDisplayName().EqualTo(RecordedAudioTrackName))
				{
					RecordedAudioTrack = Cast<UMovieSceneAudioTrack>(Track);
				}
			}

			if (!RecordedAudioTrack)
			{
				RecordedAudioTrack = MovieScene->AddTrack<UMovieSceneAudioTrack>();
				RecordedAudioTrack->SetDisplayName(RecordedAudioTrackName);
			}

			if (Settings->bReplaceRecordedAudio)
			{
				RecordedAudioTrack->RemoveAllAnimationData();
			}

			for (USoundWave* RecordedAudio : RecordedSoundWaves)
			{
				int32 RowIndex = -1;
				for (UMovieSceneSection* Section : RecordedAudioTrack->GetAllSections())
				{
					RowIndex = FMath::Max(RowIndex, Section->GetRowIndex());
				}

				UMovieSceneAudioSection* NewAudioSection = NewObject<UMovieSceneAudioSection>(RecordedAudioTrack, UMovieSceneAudioSection::StaticClass());

				FFrameRate TickResolution = RecordedAudioTrack->GetTypedOuter<UMovieScene>()->GetTickResolution();

				NewAudioSection->SetRowIndex(RowIndex + 1);
				NewAudioSection->SetSound(RecordedAudio);
				NewAudioSection->SetRange(TRange<FFrameNumber>(FFrameNumber(0), (RecordedAudio->GetDuration() * TickResolution).CeilToFrame()));

				RecordedAudioTrack->AddSection(*NewAudioSection);

				if(Settings->bAutoSaveAsset || GEditor == nullptr)
				{
					SequenceRecorderUtils::SaveAsset(RecordedAudio);
				}
			}
		}
	}

	// remove spawn delegates
	for(auto It = ActorSpawningDelegateHandles.CreateConstIterator(); It; ++It)
	{
		TWeakObjectPtr<UWorld> World = It->Key;
		if(World.IsValid())
		{
			World->RemoveOnActorSpawnedHandler(It->Value);
		}
	}

	ActorSpawningDelegateHandles.Empty();

	// also stop all dead animation recordings, i.e. ones that use GCed components
	const bool bShowMessage = false;
	FAnimationRecorderManager::Get().StopRecordingDeadAnimations(bShowMessage);

	for(USequenceRecordingBase* Recording : QueuedRecordings)
	{
		SlowTask.EnterProgressFrame();

		Recording->StopRecording(CurrentSequence.Get(), CurrentTime);
	}

	for(USequenceRecordingBase* Recording : DeadRecordings)
	{
		SlowTask.EnterProgressFrame();

		Recording->StopRecording(CurrentSequence.Get(), CurrentTime);
	}

	DeadRecordings.Empty();

	// Remove any spawned recordings
	TArray<UActorRecording*, TInlineAllocator<32>> ToRemove;
	for (UActorRecording* QueuedRecording : QueuedActorRecordings)
	{
		if (QueuedRecording->bWasSpawnedPostRecord)
		{
			ToRemove.Add(QueuedRecording);
		}
	}
	for (UActorRecording* QueuedRecording : ToRemove)
	{
		RemoveQueuedRecording(QueuedRecording);
	}


	// Stop any level sequences that were triggered
	for(int32 DupActorToTriggerIndex = 0; DupActorToTriggerIndex < DupActorsToTrigger.Num(); ++DupActorToTriggerIndex )
	{
		if (DupActorsToTrigger[DupActorToTriggerIndex].IsValid())
		{
			ALevelSequenceActor* DupActorToTrigger = DupActorsToTrigger[DupActorToTriggerIndex].Get();
			ULevelSequencePlayer* SequencePlayer = DupActorToTrigger->GetSequencePlayer();
			if (SequencePlayer)
			{
				SequencePlayer->SetDisableCameraCuts(false);
				SequencePlayer->Stop();
			}
			if (DupActorToTrigger->GetWorld())
			{
				DupActorToTrigger->GetWorld()->DestroyActor(DupActorToTrigger);
			}
		}
	}

	DupActorsToTrigger.Empty();
	CurrentTime = 0.0f;
	CurrentDelay = 0.0f;

	// Restore our cached Global Time Dilation in case they are still running the game.
	if (CurrentRecordingWorld.IsValid())
	{
		AWorldSettings* WorldSettings = CurrentRecordingWorld->GetWorldSettings();
		if (WorldSettings)
		{
			WorldSettings->SetTimeDilation(CachedGlobalTimeDilation);
		}

		CurrentRecordingWorld.Reset();
	}

	if(Settings->bCreateLevelSequence)
	{
		if(LevelSequence)
		{
			FGuid RecordedCameraGuid;
			FMovieSceneSequenceID SequenceID = MovieSceneSequenceID::Root;
			
			for(UActorRecording* Recording : QueuedActorRecordings)
			{
				if (Recording->IsActive())
				{
					AActor* ActorToRecord = Recording->GetActorToRecord();
					if (ActorToRecord && ActorToRecord->IsA<ACameraActor>())
					{
						ULevelSequence* RecordedCameraLevelSequence = Recording->GetActiveLevelSequence(LevelSequence);
						RecordedCameraGuid = Recording->GetActorInSequence(ActorToRecord, RecordedCameraLevelSequence);

						if (RecordedCameraLevelSequence != LevelSequence)
						{
							UMovieSceneCompiledDataManager* DataManager = UMovieSceneCompiledDataManager::GetPrecompiledData();
							FMovieSceneCompiledDataID       DataID      = DataManager->Compile(LevelSequence);

							if (const FMovieSceneSequenceHierarchy* Hierarchy = DataManager->FindHierarchy(DataID))
							{
								for (auto& Pair : Hierarchy->AllSubSequenceData())
								{
									if (Pair.Value.Sequence == RecordedCameraLevelSequence)
									{
										SequenceID = Pair.Key;
										break;
									}
								}
							}
						}

						break;
					}
				}
			}

			// set movie scene playback range to encompass all sections
			SequenceRecorderUtils::ExtendSequencePlaybackRange(CurrentSequence.Get());

			SequenceRecorderUtils::CreateCameraCutTrack(CurrentSequence.Get(), RecordedCameraGuid, SequenceID);

			// Stop referencing the sequence so we are listed as 'not recording'
			CurrentSequence = nullptr;

			if(Settings->bAutoSaveAsset || GEditor == nullptr)
			{
				SequenceRecorderUtils::SaveAsset(LevelSequence);
			}

			if(FSlateApplication::IsInitialized() && GIsEditor)
			{
				const FText NotificationText = FText::Format(LOCTEXT("RecordSequence", "'{0}' has been successfully recorded."), FText::FromString(LevelSequence->GetName()));
					
				FNotificationInfo Info(NotificationText);
				Info.ExpireDuration = 8.0f;
				Info.bUseLargeFont = false;
				Info.Hyperlink = FSimpleDelegate::CreateLambda([=]()
				{
					TArray<UObject*> Assets;
					Assets.Add(LevelSequence);
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(Assets);
				});
				Info.HyperlinkText = FText::Format(LOCTEXT("OpenNewAnimationHyperlink", "Open {0}"), FText::FromString(LevelSequence->GetName()));
				TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
				if ( Notification.IsValid() )
				{
					Notification->SetCompletionState( SNotificationItem::CS_Success );
				}
			}

			UE_LOGF(LogAnimation, Display, "Stopped recording sequence %ls", *LevelSequence->GetPathName());

			if (OnRecordingFinishedDelegate.IsBound())
			{
				OnRecordingFinishedDelegate.Broadcast(LevelSequence);
			}
			
			// Restart the recording if it's allowed, ie. the user has not pressed stop
			if (bAllowLooping)
			{
				StartRecording();
			}

			return true;
		}
	}
	else
	{
		UE_LOGF(LogAnimation, Display, "Stopped recording, no sequence created");
		return true;
	}

	return false;
}

bool FSequenceRecorder::IsDelaying() const
{
	return CurrentDelay > 0.0f;
}

float FSequenceRecorder::GetCurrentDelay() const
{
	return CurrentDelay;
}

bool FSequenceRecorder::IsActorValidForRecording(AActor* Actor)
{
	check(Actor);

	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

	float Distance = Settings->NearbyActorRecordingProximity;

	// check distance if valid
	if(Settings->bRecordNearbySpawnedActors && Distance > 0.0f)
	{
		const FTransform ActorTransform = Actor->GetTransform();
		const FVector ActorTranslation = ActorTransform.GetTranslation();

		for(UActorRecording* Recording : QueuedActorRecordings)
		{
			if(AActor* OtherActor = Recording->GetActorToRecord())
			{
				if(OtherActor != Actor)
				{
					const FTransform OtherActorTransform = OtherActor->GetTransform();
					const FVector OtherActorTranslation = OtherActorTransform.GetTranslation();

					if((OtherActorTranslation - ActorTranslation).Size() < Distance)
					{
						return true;
					}
				}
			}
		}
	}

	// check class if any
	for(const TSubclassOf<AActor>& ActorClass : Settings->ActorFilter.ActorClassesToRecord)
	{
		if(*ActorClass != nullptr && Actor->IsA(*ActorClass))
		{
			return true;
		}
	}

	return false;
}

void FSequenceRecorder::HandleActorSpawned(AActor* Actor)
{
	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

	if(Actor && FSequenceRecorder::Get().IsRecording())
	{
		if (UActorRecording::IsRelevantForRecording(Actor) || IsActorValidForRecording(Actor))
		{
			UActorRecording* NewRecording = AddNewQueuedRecording(Actor);
			NewRecording->bWasSpawnedPostRecord = true;
			NewRecording->StartRecording(CurrentSequence.Get(), CurrentTime, PathToRecordTo, SequenceName);
		}
	}
}

void FSequenceRecorder::HandleActorDespawned(AActor* Actor)
{
	if(Actor && FSequenceRecorder::Get().IsRecording())
	{
		for(int32 Index = 0; Index < QueuedRecordings.Num(); ++Index)
		{
			USequenceRecordingBase* Recording = QueuedRecordings[Index];
			if(Recording->GetObjectToRecord() == Actor)
			{
				UActorRecording* ActorRecording = CastChecked<UActorRecording>(Recording);
				ActorRecording->InvalidateObjectToRecord();
				DeadRecordings.Add(Recording);
				QueuedActorRecordings.RemoveSwap(ActorRecording);
				QueuedRecordings.RemoveAt(Index);
				break;
			}
		}
	}
}

void FSequenceRecorder::RefreshNextSequence()
{
	if (SequenceName.IsEmpty())
	{
		SequenceName = GetSequenceRecordingName().Len() > 0 ? GetSequenceRecordingName() : TEXT("RecordedSequence");
	}

	FString BasePath = GetSequenceRecordingBasePath() / SequenceName;

	// Cache the name of the next sequence we will try to record to
	NextSequenceName = SequenceRecorderUtils::MakeNewAssetName(BasePath, SequenceName);
}

void FSequenceRecorder::ForceRefreshNextSequence()
{
	SequenceName = GetSequenceRecordingName().Len() > 0 ? GetSequenceRecordingName() : TEXT("RecordedSequence");

	FString BasePath = GetSequenceRecordingBasePath() / SequenceName;

	// Cache the name of the next sequence we will try to record to
	NextSequenceName = SequenceRecorderUtils::MakeNewAssetName(BasePath, SequenceName);
}

TWeakObjectPtr<ASequenceRecorderGroup> FSequenceRecorder::GetRecordingGroupActor()
{
	if (CachedRecordingActor.IsValid())
	{
		return CachedRecordingActor;
	}

	// Check the map for one
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	ASequenceRecorderGroup* GroupActor = nullptr;

	if (EditorWorld && EditorWorld->PersistentLevel)
	{
		for (int32 ActorIndex = 0; ActorIndex < EditorWorld->PersistentLevel->Actors.Num(); ++ActorIndex)
		{
			AActor* Actor = EditorWorld->PersistentLevel->Actors[ActorIndex];
			GroupActor = Cast<ASequenceRecorderGroup>(Actor);
			if (GroupActor)
			{
				// We want to find the first actor
				break;
			}
		}
	}

	// We may not have one, or we may be in a situation where we can't safely create
	// an actor, calling functions should expect this to possibly be null.
	CachedRecordingActor = GroupActor;
	return CachedRecordingActor;
}

TWeakObjectPtr<USequenceRecorderActorGroup> FSequenceRecorder::AddRecordingGroup()
{
	const FScopedTransaction Transaction(LOCTEXT("AddRecordingGroup", "Add Actor Recording Group"));

	TWeakObjectPtr<ASequenceRecorderGroup> GroupActor = GetRecordingGroupActor();
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();

	FDirectoryPath ExistingBasePath;
	if (GetCurrentRecordingGroup().IsValid())
	{
		ExistingBasePath = GetCurrentRecordingGroup().Get()->SequenceRecordingBasePath;
	}

	// There may not be a group actor in the level yet, so we'll spawn a new one.
	if (!GroupActor.IsValid())
	{
		GroupActor = (ASequenceRecorderGroup*)GEditor->AddActor(EditorWorld->PersistentLevel, ASequenceRecorderGroup::StaticClass(), FTransform::Identity);
		CachedRecordingActor = GroupActor;
	}

	// Now add a new actor group to this actor
	check(GroupActor.IsValid());
	USequenceRecorderActorGroup* ActorGroup = NewObject<USequenceRecorderActorGroup>(GroupActor.Get(), NAME_None, RF_Transactional);
	if (!ExistingBasePath.Path.IsEmpty())
	{
		ActorGroup->SequenceRecordingBasePath = ExistingBasePath;
	}

	FString NewName = SequenceRecorderUtils::MakeNewGroupName(*ActorGroup->SequenceRecordingBasePath.Path, TEXT("Setup"), GetRecordingGroupNames());
	ActorGroup->GroupName = FName(*NewName);
	ActorGroup->SequenceName = NewName;
	GroupActor->ActorGroups.Add(ActorGroup);

	// Remove the existing queued recordings which marks us as dirty so the UI will refresh too.
	ClearQueuedRecordings();

	// And then select our new object by default
	CurrentRecorderGroup = ActorGroup;

	ForceRefreshNextSequence();
	
	if (OnRecordingGroupAddedDelegate.IsBound())
	{
		OnRecordingGroupAddedDelegate.Broadcast(CurrentRecorderGroup);
	}

	return CurrentRecorderGroup;
}

void FSequenceRecorder::RemoveCurrentRecordingGroup()
{
	if (!GetCurrentRecordingGroup().IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("RemoveActorRecordingGroup", "Remove Actor Recording Group"));

	ClearQueuedRecordings();
	TWeakObjectPtr<ASequenceRecorderGroup> GroupActor = GetRecordingGroupActor();
	if (GroupActor.IsValid())
	{
		GroupActor->ActorGroups.Remove(GetCurrentRecordingGroup().Get());
	}
}

TWeakObjectPtr<USequenceRecorderActorGroup> FSequenceRecorder::DuplicateRecordingGroup()
{
	check(GetCurrentRecordingGroup().IsValid());
	check(GetRecordingGroupActor().IsValid());

	FString BaseName;
	if (GetCurrentRecordingGroup().IsValid())
	{
		BaseName = GetCurrentRecordingGroup().Get()->SequenceName;
	}

	const FScopedTransaction Transaction(LOCTEXT("DuplicateActorRecordingGroup", "Duplicate Actor Recording Group"));

	USequenceRecorderActorGroup* DuplicatedGroup = DuplicateObject<USequenceRecorderActorGroup>(GetCurrentRecordingGroup().Get(), GetRecordingGroupActor().Get());
	FString NewName = SequenceRecorderUtils::MakeNewGroupName(*DuplicatedGroup->SequenceRecordingBasePath.Path, BaseName, GetRecordingGroupNames());
	DuplicatedGroup->GroupName = FName(*NewName);
	DuplicatedGroup->SequenceName = NewName;
	DuplicatedGroup->TargetLevelSequence = nullptr;

	for (UActorRecording* ActorRecording : DuplicatedGroup->RecordedActors)
	{
		if (ActorRecording != nullptr)
		{
			ActorRecording->TakeNumber = 1;
		}
	}

	GetRecordingGroupActor().Get()->ActorGroups.Add(DuplicatedGroup);

	// We'll invoke the standard load function so that it triggers everything to clear/update correctly.
	TWeakObjectPtr<USequenceRecorderActorGroup> LoadedGroup = LoadRecordingGroup(DuplicatedGroup->GroupName);

	if (OnRecordingGroupAddedDelegate.IsBound())
	{
		OnRecordingGroupAddedDelegate.Broadcast(LoadedGroup);
	}

	return LoadedGroup;
}

TArray<FName> FSequenceRecorder::GetRecordingGroupNames() const
{
	TArray<FName> GroupNames;
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	if (EditorWorld && EditorWorld->PersistentLevel)
	{
		for (int32 ActorIndex = 0; ActorIndex < EditorWorld->PersistentLevel->Actors.Num(); ++ActorIndex)
		{
			AActor* Actor = EditorWorld->PersistentLevel->Actors[ActorIndex];
			ASequenceRecorderGroup* GroupActor = Cast<ASequenceRecorderGroup>(Actor);
			if (GroupActor)
			{
				for (USequenceRecorderActorGroup* ActorGroup : GroupActor->ActorGroups)
				{
					if (ActorGroup)
					{
						GroupNames.Add(ActorGroup->GroupName);
					}
				}

				// We only examine the first actor group in the map as it should contain all of our groups.
				break;
			}
		}
	}

	return GroupNames;
}

TWeakObjectPtr<USequenceRecorderActorGroup> FSequenceRecorder::LoadRecordingGroup(const FName Name)
{
	TWeakObjectPtr<ASequenceRecorderGroup> GroupActor = nullptr;
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();

	if (EditorWorld && EditorWorld->PersistentLevel)
	{
		for (int32 ActorIndex = 0; ActorIndex < EditorWorld->PersistentLevel->Actors.Num(); ++ActorIndex)
		{
			AActor* Actor = EditorWorld->PersistentLevel->Actors[ActorIndex];
			GroupActor = Cast<ASequenceRecorderGroup>(Actor);
			if (GroupActor.IsValid())
			{
				// We only examine the first actor group in the map
				break;
			}
		}
	}

	if (GroupActor.IsValid())
	{
		// Remove the existing queued recordings to mark us as dirty (this causes the UI to refresh)
		ClearQueuedRecordings();

		TWeakObjectPtr<USequenceRecorderActorGroup> ActorGroup = GroupActor->FindActorGroup(Name);
		if (ActorGroup.IsValid())
		{
			CurrentRecorderGroup = ActorGroup;
			for (UActorRecording* ActorRecording : ActorGroup->RecordedActors)
			{
				if (ActorRecording != nullptr)
				{
					ActorRecording->AddToRoot();
					QueuedRecordings.Add(ActorRecording);
					QueuedActorRecordings.Add(ActorRecording);
				}
			}
			ForceRefreshNextSequence();
			return CurrentRecorderGroup;
		}
	}

	// We either don't have a group actor or we can't find a group by that name, clear anything we have loaded.
	// This lets the UI handle switching back to profile "None".
	ClearQueuedRecordings();
	CurrentRecorderGroup = nullptr;

	// Refresh the next sequence after nulling out the recording group so we get the default name.
	ForceRefreshNextSequence();
	return nullptr;
}

FString FSequenceRecorder::GetSequenceRecordingBasePath() const
{
	TWeakObjectPtr<USequenceRecorderActorGroup> RecordingGroup = GetCurrentRecordingGroup();
	if (RecordingGroup.IsValid())
	{
		return RecordingGroup->SequenceRecordingBasePath.Path;
	}

	// If no profile is loaded, we just return the default.
	return GetDefault<USequenceRecorderActorGroup>()->SequenceRecordingBasePath.Path;
}

FString FSequenceRecorder::GetSequenceRecordingName() const
{
	TWeakObjectPtr<USequenceRecorderActorGroup> RecordingGroup = GetCurrentRecordingGroup();
	if (RecordingGroup.IsValid())
	{
		return RecordingGroup->SequenceName;
	}

	// If no profile is loaded, just return the default value.
	return GetDefault<USequenceRecorderActorGroup>()->SequenceName;
}

void FSequenceRecorder::SetImmersive()
{
	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

	if (Settings->bImmersiveMode)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedPtr< IAssetViewport > ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();

		if( ActiveLevelViewport.IsValid() )
		{
			bWasImmersive = ActiveLevelViewport->IsImmersive();

			if (!ActiveLevelViewport->IsImmersive())
			{
				const bool bWantImmersive = true;
				const bool bAllowAnimation = false;
				ActiveLevelViewport->MakeImmersive( bWantImmersive, bAllowAnimation );
			}
		}
	}

}

void FSequenceRecorder::RestoreImmersive()
{
	const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();

	if (Settings->bImmersiveMode)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedPtr< IAssetViewport > ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();

		if( ActiveLevelViewport.IsValid() )
		{
			if (ActiveLevelViewport->IsImmersive() != bWasImmersive)
			{
				const bool bAllowAnimation = false;
				ActiveLevelViewport->MakeImmersive(bWasImmersive, bAllowAnimation);
			}
		}
	}
}

void FSequenceRecorder::BuildQueuedRecordings()
{
	QueuedActorRecordings.Reset();

	for (USequenceRecordingBase* QueuedRecording : QueuedRecordings)
	{
		if (UActorRecording* ActorRecording = Cast<UActorRecording>(QueuedRecording))
		{
			QueuedActorRecordings.Add(ActorRecording);
		}
	}
	for (TSharedPtr<ISequenceRecorderExtender> RecorderExtender : SequenceRecorderExtenders)
	{
		RecorderExtender->BuildQueuedRecordings(QueuedRecordings);
	}
}

#undef LOCTEXT_NAMESPACE
