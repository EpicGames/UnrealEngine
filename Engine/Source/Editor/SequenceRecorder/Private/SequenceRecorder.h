// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Sections/MovieSceneAnimationSectionRecorder.h"
#include "LevelSequence.h"
#include "Sections/MovieScene3DTransformSectionRecorder.h"
#include "ISequenceRecorder.h"
#include "Sections/MovieSceneMultiPropertyRecorder.h"

class APlayerController;
class AActor;
class ALevelSequenceActor;
class ISequenceAudioRecorder;
class UCanvas;
class USequenceRecordingBase;
class UActorRecording;
class UTexture;
class ASequenceRecorderGroup;
class USequenceRecorderActorGroup;

struct FSequenceRecorder
{
public:
	static FName MovieScenePropertyRecorderFactoryName;

	/** Singleton accessor */
	static FSequenceRecorder& Get();

	/** Initialize any resources we need */
	void Initialize();

	/** Clear any resources we need */
	void Shutdown();

	/** Starts recording a sequence. */
	bool StartRecording(const FString& InPathToRecordTo = FString(), const FString& InSequenceName = FString());

	/** Starts recording a sequence for the specified world. */
	bool StartRecordingForReplay(UWorld* World, const struct FSequenceRecorderActorFilter& ActorFilter);

	/** Stops any currently recording sequence */
	bool StopRecording(bool bAllowLooping = false);

	/** Tick the sequence recorder */
	void Tick(float DeltaSeconds);

	/** Get whether we are currently delaying a recording */
	bool IsDelaying() const;

	/** Get the current delay we are waiting for */
	float GetCurrentDelay() const;

	TWeakObjectPtr<class ULevelSequence> GetCurrentSequence() { return CurrentSequence; }

	bool IsRecordingQueued(AActor* Actor) const;
	bool IsRecordingQueued(UObject* SequenceRecordingObjectToRecord) const;

	UActorRecording* FindRecording(AActor* Actor) const;
	USequenceRecordingBase* FindRecording(UObject* SequenceRecordingObjectToRecord) const;

	void StartAllQueuedRecordings();

	void StopAllQueuedRecordings();

	void StopRecordingDeadAnimations();

	void AddNewQueuedRecordingsForSelectedActors();

	void AddNewQueuedRecordingForCurrentPlayer();

	bool CanAddNewQueuedRecordingForCurrentPlayer() const;

	class UActorRecording* AddNewQueuedRecording(AActor* Actor = nullptr, UAnimSequence* AnimSequence = nullptr, float Length = 0.0f);
	class USequenceRecordingBase* AddNewQueuedRecording(UObject* SequenceRecordingObjectToRecord);

	void RemoveQueuedRecording(USequenceRecordingBase* Recording);

	bool HasQueuedRecordings() const;

	void ClearQueuedRecordings();

	const TArray<UActorRecording*>& GetQueuedActorRecordings() { return QueuedActorRecordings; }
	const TArray<USequenceRecordingBase*>& GetQueuedRecordings() { return QueuedRecordings; }

	bool AreQueuedRecordingsDirty() const { return bQueuedRecordingsDirty; }

	void ResetQueuedRecordingsDirty() { bQueuedRecordingsDirty = false; }

	bool IsRecording() const;

	/** Draw the countdown to the screen */
	void DrawDebug(UCanvas* InCanvas, APlayerController* InPlayerController);

	/** Handle actors being spawned */
	void HandleActorSpawned(AActor* Actor);

	/** Handle actors being de-spawned */
	void HandleActorDespawned(AActor* Actor);

	TWeakObjectPtr<USequenceRecorderActorGroup> GetCurrentRecordingGroup() const
	{
		return CurrentRecorderGroup;
	}

	TWeakObjectPtr<ASequenceRecorderGroup> GetRecordingGroupActor();

	TWeakObjectPtr<USequenceRecorderActorGroup> AddRecordingGroup();
	void RemoveCurrentRecordingGroup();
	TWeakObjectPtr<USequenceRecorderActorGroup> DuplicateRecordingGroup();
	TWeakObjectPtr<USequenceRecorderActorGroup> LoadRecordingGroup(const FName Name);

	TArray<FName> GetRecordingGroupNames() const;

	FString GetSequenceRecordingBasePath() const;
	FString GetSequenceRecordingName() const;

	TArray<TSharedPtr<ISequenceRecorderExtender>>& GetSequenceRecorderExtenders() { return SequenceRecorderExtenders; }

	/** Get the built-in animation factory (as this uses special case handling) */
	const FMovieSceneAnimationSectionRecorderFactory& GetAnimationRecorderFactory() const
	{
		return AnimationSectionRecorderFactory;
	}

	/** Get the built-in transform factory (as this uses special case handling) */
	const FMovieScene3DTransformSectionRecorderFactory& GetTransformRecorderFactory() const
	{
		return TransformSectionRecorderFactory;
	}

	/** Get the name of the next sequence we are targeting */
	const FString& GetNextSequenceName() const { return NextSequenceName; }

	/** Refresh the name of the next sequence we will be recording */
	void RefreshNextSequence();

	/** Force refresh the name of the next sequence, disregard the current sequence name */
	void ForceRefreshNextSequence();

	/** Multicast delegate fired when recording is started */
	FOnRecordingStarted OnRecordingStartedDelegate;

	/** Multicast delegate fired when recording has finished */
	FOnRecordingFinished OnRecordingFinishedDelegate;

	/** Multicast delegate fired when a recording group has been added */
	FOnRecordingGroupAdded OnRecordingGroupAddedDelegate;

private:
	/** Starts recording a sequence, possibly delayed */
	bool StartRecordingInternal(UWorld* World);

	/** Check if an actor is valid for recording */
	bool IsActorValidForRecording(AActor* Actor);

	/** Handle exiting cleanly from PIE */
	void HandleEndPIE(bool bSimulating);

	/** Set immersive mode and store whether viewports were immersive */
	void SetImmersive();

	/** Restore immersive mode to stored value */
	void RestoreImmersive();

	void BuildQueuedRecordings();

private:
	/** Constructor, private - use Get() function */
	FSequenceRecorder();

	/** Currently recording level sequence, if any */
	TWeakObjectPtr<class ULevelSequence> CurrentSequence;

	/** World we are recording a replay for, if any. Is only valid if we're recording from Replay network drivers.*/
	TWeakObjectPtr<class UWorld> CurrentReplayWorld;

	/** Actor World that our last started recording was. Null if there is not a recording in progress. */
	TWeakObjectPtr<class UWorld> CurrentRecordingWorld;

	/** Recorder Group that our actor recordings go into.  */
	TWeakObjectPtr<USequenceRecorderActorGroup> CurrentRecorderGroup;

	/** Cached actor for this level who holds the recording group. */
	TWeakObjectPtr<ASequenceRecorderGroup> CachedRecordingActor;

	TArray<UActorRecording*> QueuedActorRecordings;

	TArray<USequenceRecordingBase*> QueuedRecordings;

	TArray<USequenceRecordingBase*> DeadRecordings;

	bool bQueuedRecordingsDirty;

	bool bWasImmersive;

	/**  Recorder Extenders  */
	TArray<TSharedPtr<ISequenceRecorderExtender>> SequenceRecorderExtenders;

	/** The delay we are currently waiting for */
	float CurrentDelay;

	/** Current recording time */
	float CurrentTime;

	/** Cached Global Time dilation, used to restore previous time dilation after recording stops. */
	float CachedGlobalTimeDilation;

	/** Delegate handles for FOnActorSpawned events */
	TMap<TWeakObjectPtr<UWorld>, FDelegateHandle> ActorSpawningDelegateHandles;

	/** Texture we use for the countdown */
	TWeakObjectPtr<UTexture> CountdownTexture;

	/** Texture we use for the recording indicator */
	TWeakObjectPtr<UTexture> RecordingIndicatorTexture;

	/** Cached sequence name to record to */
	FString SequenceName;

	/** The next sequence we will be targeting. Name can change depending on assets being deleted, moved, renamed etc. */
	FString NextSequenceName;

	/** Cached sequence path to record to */
	FString PathToRecordTo;

	/** Built-in animation recorder factory */
	FMovieSceneAnimationSectionRecorderFactory AnimationSectionRecorderFactory;

	/** Built-in transform recorder factory */
	FMovieScene3DTransformSectionRecorderFactory TransformSectionRecorderFactory;

	/** Audio recorder */
	TUniquePtr<ISequenceAudioRecorder> AudioRecorder;

	/** Built in multi-property recorder */
	FMovieSceneMultiPropertyRecorderFactory MultiPropertySectionRecorder;

	/** Duplicated level sequence actors to trigger, to be stopped at the end of recording */
	TArray<TWeakObjectPtr<ALevelSequenceActor>> DupActorsToTrigger;
};
