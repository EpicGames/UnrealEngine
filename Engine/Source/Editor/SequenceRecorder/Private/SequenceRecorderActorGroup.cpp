// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceRecorderActorGroup.h"
#include "SequenceRecorder.h"
#include "ActorRecording.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequenceRecorderActorGroup)

void USequenceRecorderActorGroup::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(USequenceRecorderActorGroup, SequenceName))
		{
			GroupName = *SequenceName;
			TargetLevelSequence = nullptr;

			for (UActorRecording* ActorRecording : RecordedActors)
			{
				if (ActorRecording != nullptr)
				{
					ActorRecording->TakeNumber = 1;
				}
			}
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(USequenceRecorderActorGroup, SequenceName) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(USequenceRecorderActorGroup, SequenceRecordingBasePath))
		{
			FSequenceRecorder::Get().ForceRefreshNextSequence();
		}
	}
}
