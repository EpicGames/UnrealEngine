// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceRecorderBlueprintLibrary.h"
#include "SequenceRecorder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequenceRecorderBlueprintLibrary)

void USequenceRecorderBlueprintLibrary::StartRecordingSequence(const TArray<AActor*>& ActorsToRecord)
{
	FSequenceRecorder::Get().ClearQueuedRecordings();
	for(AActor* Actor : ActorsToRecord)
	{
		FSequenceRecorder::Get().AddNewQueuedRecording(Actor);
	}
	
	FSequenceRecorder::Get().StartRecording();
}

bool USequenceRecorderBlueprintLibrary::IsRecordingSequence()
{
	return FSequenceRecorder::Get().IsRecording();
}

void USequenceRecorderBlueprintLibrary::StopRecordingSequence()
{
	FSequenceRecorder::Get().StopRecording();
}
