// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceRecorderCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#define LOCTEXT_NAMESPACE "SequenceRecorderCommands"

FSequenceRecorderCommands::FSequenceRecorderCommands()
	: TCommands<FSequenceRecorderCommands>("SequenceRecorder.Common", LOCTEXT("SequenceRecorderCommandsLabel", "Sequence Recorder"), NAME_None, FAppStyle::GetAppStyleSetName())
{
}

void FSequenceRecorderCommands::RegisterCommands()
{
	UI_COMMAND(RecordAll, "Record", "Record all recordings", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::R));
	UI_COMMAND(StopAll, "StopAll", "Stop all recordings", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::S));
	UI_COMMAND(AddRecording, "Add", "Add a new recording for selected actors. If nothing selected, an empty recording will be added", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddCurrentPlayerRecording, "Add Player", "Add a new recording for the current player.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveRecording, "Remove", "Remove selected recording", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveAllRecordings, "Remove All", "Remove all recordings", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddRecordingGroup, "Add Group", "Adds a new group for Actor recordings to be stored with", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveRecordingGroup, "Remove Group", "Removes the currently selected recording group", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DuplicateRecordingGroup, "Duplicate Group", "Duplicates the current recording group", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
