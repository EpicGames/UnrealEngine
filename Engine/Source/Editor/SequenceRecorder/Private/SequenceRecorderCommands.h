// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"

class FUICommandInfo;

class FSequenceRecorderCommands : public TCommands<FSequenceRecorderCommands>
{
public:
	FSequenceRecorderCommands();

	TSharedPtr<FUICommandInfo> RecordAll;
	TSharedPtr<FUICommandInfo> StopAll;
	TSharedPtr<FUICommandInfo> AddRecording;
	TSharedPtr<FUICommandInfo> AddCurrentPlayerRecording;
	TSharedPtr<FUICommandInfo> RemoveRecording;
	TSharedPtr<FUICommandInfo> RemoveAllRecordings;
	TSharedPtr<FUICommandInfo> AddRecordingGroup;
	TSharedPtr<FUICommandInfo> RemoveRecordingGroup;
	TSharedPtr<FUICommandInfo> DuplicateRecordingGroup;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};
