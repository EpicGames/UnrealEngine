// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceRecorderDetailsCustomization.h"

#include "DetailLayoutBuilder.h"
#include "ISequenceRecorder.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "SequenceRecorderSettings.h"

void FSequenceRecorderDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	ISequenceRecorder& RecorderModule = FModuleManager::Get().LoadModuleChecked<ISequenceRecorder>("SequenceRecorder");
	if (!RecorderModule.HasAudioRecorder())
	{
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(USequenceRecorderSettings, RecordAudio));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(USequenceRecorderSettings, AudioGain));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(USequenceRecorderSettings, AudioSubDirectory));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(USequenceRecorderSettings, bSplitAudioChannelsIntoSeparateTracks));
	}
}
