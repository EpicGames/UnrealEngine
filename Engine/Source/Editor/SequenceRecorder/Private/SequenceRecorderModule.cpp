// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorRecording.h"
#include "ActorRecordingDetailsCustomization.h"
#include "Animation/AnimationRecordingSettings.h"
#include "Animation/AnimSequence.h"
#include "AnimationRecorder.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Debug/DebugDrawService.h"
#include "Engine/Level.h"
#include "EngineGlobals.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"
#include "ISequenceAudioRecorder.h"
#include "ISequenceRecorder.h"
#include "IStructureDetailsView.h"
#include "LevelEditor.h"
#include "LevelSequence.h"
#include "Misc/CoreMisc.h"
#include "Misc/ExpressionParserTypes.h"
#include "Misc/Guid.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneTimeHelpers.h"
#include "PersonaModule.h"
#include "PropertiesToRecordForClassDetailsCustomization.h"
#include "PropertyEditorModule.h"
#include "SequenceRecorder.h"
#include "SequenceRecorderCommands.h"
#include "SequenceRecorderDetailsCustomization.h"
#include "SequenceRecorderSettings.h"
#include "SequenceRecorderUtils.h"
#include "SSequenceRecorder.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "TimecodeBoneMethodCustomization.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "SequenceRecorder"

static const FName SequenceRecorderTabName(TEXT("SequenceRecorder"));

static TAutoConsoleVariable<float> CVarDefaultRecordedAnimLength(
	TEXT("AnimRecorder.AnimLength"),
	FAnimationRecordingSettings::DefaultMaximumLength,
	TEXT("Sets default animation length for the animation recorder system."),
	ECVF_Default);

static FAutoConsoleCommand CVarAnimRecorderSampleFrameRate(
	TEXT("AnimRecorder.SampleRate"),
	TEXT("Argument: valid Frame Rate format\n")
	TEXT("Sets the sample frame-rate for the animation recorder system"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		bool bValidFrameRate = false;
		if (Args.Num() == 1)
		{
			TValueOrError<FFrameRate, FExpressionError> ParseResult = ParseFrameRate(*Args[0]);

			if (ParseResult.IsValid())
			{
				GetMutableDefault<USequenceRecorderSettings>()->DefaultAnimationSettings.SampleFrameRate = ParseResult.GetValue();
				UE_LOGF(LogEngine, Display, "Sequence recorder default sample frame rate set to: %ls", *(GetMutableDefault<USequenceRecorderSettings>()->DefaultAnimationSettings.SampleFrameRate.ToPrettyText().ToString()));
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "Invalid Frame Rate format provided for AnimRecorder.SampleRate:\n%ls", *ParseResult.GetError().Text.ToString());
			}
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Missing Frame Rate argument for AnimRecorder.SampleRate");
		}
	}),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarAnimRecorderWorldSpace(
	TEXT("AnimRecorder.RecordInWorldSpace"),
	1,
	TEXT("True to record anim keys in world space, false to record only in local space."),
	ECVF_Default);

class FSequenceRecorderSettingsTabFactory : public FWorkflowTabFactory
{
public:
	FSequenceRecorderSettingsTabFactory(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
		: FWorkflowTabFactory(TEXT("PersonaSequenceRecorderSettings"), InHostingApp)
	{
		TabLabel = LOCTEXT("AnimationRecordingSettings", "Recording Settings");
		TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "SequenceRecorder.TabIcon");
		ViewMenuDescription = LOCTEXT("AnimationRecordingSettings", "Recording Settings");
		ViewMenuTooltip = LOCTEXT("AnimationRecordingSettings_Tooltip", "Settings for animation recording");

		StructOnScope = MakeShared<FStructOnScope>(FAnimationRecordingSettings::StaticStruct(), (uint8*)&GetMutableDefault<USequenceRecorderSettings>()->DefaultAnimationSettings);
	}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		FStructureDetailsViewArgs StructureDetailsViewArgs;
		return PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureDetailsViewArgs, StructOnScope, LOCTEXT("AnimationRecordingSettings", "Recording Settings"))->GetWidget().ToSharedRef();
	}
	
	TSharedPtr<FStructOnScope> StructOnScope;
};

class FSequenceRecorderModule : public ISequenceRecorder, private FSelfRegisteringExec
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
#if WITH_EDITOR
		GetMutableDefault<USequenceRecorderSettings>()->LoadConfig();

		// set cvar defaults
		CVarDefaultRecordedAnimLength.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
		{
			GetMutableDefault<USequenceRecorderSettings>()->DefaultAnimationSettings.Length = CVarDefaultRecordedAnimLength.GetValueOnGameThread();
		}));

		CVarAnimRecorderWorldSpace.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
		{
			GetMutableDefault<USequenceRecorderSettings>()->DefaultAnimationSettings.bRecordInWorldSpace = (CVarAnimRecorderWorldSpace.GetValueOnGameThread() != 0);
		}));

		FSequenceRecorderCommands::Register();

		// init sequence recorder
		FSequenceRecorder::Get().Initialize();

		// register main tick
		if(GEngine)
		{
			PostEditorTickHandle = GEngine->OnPostEditorTick().AddStatic(&FSequenceRecorderModule::TickSequenceRecorder);
		}

		if (GEditor)
		{
			// register Persona recorder
			FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>(TEXT("Persona"));
			PersonaModule.OnIsRecordingActive().BindStatic(&FSequenceRecorderModule::HandlePersonaIsRecordingActive);
			PersonaModule.OnRecord().BindStatic(&FSequenceRecorderModule::HandlePersonaRecord);
			PersonaModule.OnStopRecording().BindStatic(&FSequenceRecorderModule::HandlePersonaStopRecording);
			PersonaModule.OnGetCurrentRecording().BindStatic(&FSequenceRecorderModule::HandlePersonaCurrentRecording);
			PersonaModule.OnGetCurrentRecordingTime().BindStatic(&FSequenceRecorderModule::HandlePersonaCurrentRecordingTime);
			PersonaRegisterTabsHandle = PersonaModule.OnRegisterTabs().AddLambda([](FWorkflowAllowedTabSet& InWorkflowAllowedTabSet, TSharedPtr<FAssetEditorToolkit> InHostingApp)
			{
				InWorkflowAllowedTabSet.RegisterFactory(MakeShared<FSequenceRecorderSettingsTabFactory>(InHostingApp));
			});
			PersonaLayoutExtensionsHandle = PersonaModule.OnRegisterLayoutExtensions().AddLambda([](FLayoutExtender& InExtender)
			{
				InExtender.ExtendLayout(FTabId(TEXT("AdvancedPreviewTab")), ELayoutExtensionPosition::After, FTabManager::FTab(FTabId(TEXT("PersonaSequenceRecorderSettings")), ETabState::ClosedTab));
			});

			// register 'keep simulation changes' recorder
			FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
			LevelEditorModule.OnCaptureSingleFrameAnimSequence().BindStatic(&FSequenceRecorderModule::HandleCaptureSingleFrameAnimSequence);

			// register standalone UI
			auto RegisterTabSpawner = [this]()
			{
				FTabSpawnerEntry& Entry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(SequenceRecorderTabName, FOnSpawnTab::CreateStatic(&FSequenceRecorderModule::SpawnSequenceRecorderTab))
				.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCinematicsCategory())
				.SetDisplayName(LOCTEXT("SequenceRecorderTabTitle", "Sequence Recorder"))
				.SetTooltipText(LOCTEXT("SequenceRecorderTooltipText", "Open the Sequence Recorder tab."))
				.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "SequenceRecorder.TabIcon"));

				TabSpawnerEntry = &Entry;
				TabSpawnerEntry->SetMenuType(ETabSpawnerMenuType::Hidden);
			};
			FLevelEditorModule* LocalLevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor"));
			if (LocalLevelEditorModule && LocalLevelEditorModule->GetLevelEditorTabManager())
			{
				RegisterTabSpawner();
			}
			else
			{
				LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda(RegisterTabSpawner);
			}

			// register for debug drawing
			DrawDebugDelegateHandle = UDebugDrawService::Register(TEXT("Decals"), FDebugDrawDelegate::CreateStatic(&FSequenceRecorderModule::DrawDebug));

			// register details customization
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.RegisterCustomClassLayout(UActorRecording::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FActorRecordingDetailsCustomization::MakeInstance));
			PropertyModule.RegisterCustomClassLayout(USequenceRecorderSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FSequenceRecorderDetailsCustomization::MakeInstance));
			PropertyModule.RegisterCustomPropertyTypeLayout(FPropertiesToRecordForClass::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPropertiesToRecordForClassDetailsCustomization::MakeInstance));
			PropertyModule.RegisterCustomPropertyTypeLayout(FPropertiesToRecordForActorClass::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPropertiesToRecordForActorClassDetailsCustomization::MakeInstance));
			PropertyModule.RegisterCustomPropertyTypeLayout("TimecodeBoneMethod", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTimecodeBoneMethodCustomization::MakeInstance));
		}
#endif
	}

	virtual void ShutdownModule() override
	{
#if WITH_EDITOR

		FSequenceRecorder::Get().Shutdown();

		if (GEditor)
		{
			UDebugDrawService::Unregister(DrawDebugDelegateHandle);

			if (FSlateApplication::IsInitialized())
			{
				FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(SequenceRecorderTabName);
			}

			if(FModuleManager::Get().IsModuleLoaded(TEXT("LevelEditor")))
			{
				FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
				LevelEditorModule.OnCaptureSingleFrameAnimSequence().Unbind();
				LevelEditorModule.OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);
			}

			if (FModuleManager::Get().IsModuleLoaded(TEXT("Persona")))
			{
				FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>(TEXT("Persona"));
				PersonaModule.OnIsRecordingActive().Unbind();
				PersonaModule.OnRecord().Unbind();
				PersonaModule.OnStopRecording().Unbind();
				PersonaModule.OnGetCurrentRecording().Unbind();
				PersonaModule.OnGetCurrentRecordingTime().Unbind();
				PersonaModule.OnRegisterTabs().Remove(PersonaRegisterTabsHandle);
				PersonaModule.OnRegisterLayoutExtensions().Remove(PersonaLayoutExtensionsHandle);
			}

			if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
			{
				FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

				if (UObjectInitialized())
				{
					PropertyModule.UnregisterCustomClassLayout(UActorRecording::StaticClass()->GetFName());
					PropertyModule.UnregisterCustomClassLayout(USequenceRecorderSettings::StaticClass()->GetFName());
					PropertyModule.UnregisterCustomPropertyTypeLayout(FPropertiesToRecordForClass::StaticStruct()->GetFName());
					PropertyModule.UnregisterCustomPropertyTypeLayout("TimecodeBoneMethod");
				}
			}
		}

		if(GEngine)
		{
			GEngine->OnPostEditorTick().Remove(PostEditorTickHandle);
		}
#endif
	}

	// FSelfRegisteringExec implementation
	virtual bool Exec_Editor(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
#if WITH_EDITOR
		if (FParse::Command(&Cmd, TEXT("RecordAnimation")))
		{
			return HandleRecordAnimationCommand(InWorld, Cmd, Ar);
		}
		else if (FParse::Command(&Cmd, TEXT("StopRecordingAnimation")))
		{
			return HandleStopRecordAnimationCommand(InWorld, Cmd, Ar);
		}
		else if (FParse::Command(&Cmd, TEXT("RecordSequence")))
		{
			return HandleRecordSequenceCommand(InWorld, Cmd, Ar);
		}
		else if (FParse::Command(&Cmd, TEXT("StopRecordingSequence")))
		{
			return HandleStopRecordSequenceCommand(InWorld, Cmd, Ar);
		}
#endif
		return false;
	}

	static AActor* FindActorByName(const FString& ActorNameStr, UWorld* InWorld)
	{
		for (ULevel const* Level : InWorld->GetLevels())
		{
			if (Level)
			{
				for (AActor* Actor : Level->Actors)
				{
					if (Actor)
					{
						if (Actor->GetName() == ActorNameStr)
						{
							return Actor;
						}
					}
				}
			}
		}

		return nullptr;
	}

	static bool HandleRecordAnimationCommand(UWorld* InWorld, const TCHAR* InStr, FOutputDevice& Ar)
	{
#if WITH_EDITOR
		const TCHAR* Str = InStr;
		// parse actor name
		TCHAR ActorName[128];
		AActor* FoundActor = nullptr;
		if (FParse::Token(Str, ActorName, UE_ARRAY_COUNT(ActorName), 0))
		{
			FoundActor = FindActorByName(FString(ActorName), InWorld);
		}

		if (FoundActor)
		{
			USkeletalMeshComponent* const SkelComp = FoundActor->FindComponentByClass<USkeletalMeshComponent>();
			if (SkelComp)
			{
				TCHAR AssetPath[256];
				FParse::Token(Str, AssetPath, UE_ARRAY_COUNT(AssetPath), 0);
				FString const AssetName = FPackageName::GetLongPackageAssetName(AssetPath);
				return FAnimationRecorderManager::Get().RecordAnimation(SkelComp, AssetPath, AssetName, GetDefault<USequenceRecorderSettings>()->DefaultAnimationSettings);
			}
		}
#endif
		return false;
	}

	static AActor* FindActorByLabel(const FString& ActorNameStr, UWorld* InWorld, bool bFuzzy = false)
	{
		// search for the actor by name
		for (ULevel* Level : InWorld->GetLevels())
		{
			if (Level)
			{
				for (AActor* Actor : Level->Actors)
				{
					if (Actor)
					{
						if (Actor->GetActorLabel() == ActorNameStr)
						{
							return Actor;
						}
					}
				}
			}
		}

		// if we want to do a fuzzy search then we return the first actor whose name that starts 
		// the specified string
		if(bFuzzy)
		{
			for (ULevel* Level : InWorld->GetLevels())
			{
				if (Level)
				{
					for (AActor* Actor : Level->Actors)
					{
						if (Actor)
						{
							if (Actor->GetActorLabel().StartsWith(ActorNameStr))
							{
								return Actor;
							}
						}
					}
				}
			}			
		}

		return nullptr;
	}

	static bool HandleStopRecordAnimationCommand(UWorld* InWorld, const TCHAR* InStr, FOutputDevice& Ar)
	{
#if WITH_EDITOR
		const TCHAR* Str = InStr;

		// parse actor name
		TCHAR ActorName[128];
		AActor* FoundActor = nullptr;
		bool bStopAll = false;
		if (FParse::Token(Str, ActorName, UE_ARRAY_COUNT(ActorName), 0))
		{
			FString const ActorNameStr = FString(ActorName);

			if (ActorNameStr.ToLower() == TEXT("all"))
			{
				bStopAll = true;
			}
			else if (InWorld)
			{
				FoundActor = FindActorByName(ActorNameStr, InWorld);
			}
		}

		if (bStopAll)
		{
			FAnimationRecorderManager::Get().StopRecordingAllAnimations();
			return true;
		}
		else if (FoundActor)
		{
			USkeletalMeshComponent* const SkelComp = FoundActor->FindComponentByClass<USkeletalMeshComponent>();
			if (SkelComp)
			{
				FAnimationRecorderManager::Get().StopRecordingAnimation(SkelComp);
				return true;
			}
		}

#endif
		return false;
	}

	static void FindActorsOfClass(UClass* Class, UWorld* InWorld, TArray<AActor*>& OutActors)
	{
		for (ULevel* Level : InWorld->GetLevels())
		{
			if (Level)
			{
				for (AActor* Actor : Level->Actors)
				{
					if (Actor && Actor->IsA(Class) && UActorRecording::IsRelevantForRecording(Actor))
					{
						OutActors.AddUnique(Actor);
					}
				}
			}
		}
	}

	static bool HandleRecordSequenceCommand(UWorld* InWorld, const TCHAR* InStr, FOutputDevice& Ar)
	{
#if WITH_EDITOR
		USequenceRecorderSettings* Settings = GetMutableDefault<USequenceRecorderSettings>();

		enum class EFilterType : int32
		{
			None,
			All,
			Actor,
			Class
		};

		const TCHAR* Str = InStr;
		EFilterType FilterType = EFilterType::None;
		TCHAR Filter[128];
		if(FParse::Token(Str, Filter, UE_ARRAY_COUNT(Filter), 0))
		{
			FString const FilterStr = Filter;
			if (FilterStr == TEXT("all"))
			{
				FilterType = EFilterType::All;
			}
			else if(FilterStr == TEXT("actor"))
			{
				FilterType = EFilterType::Actor;
			}
			else if(FilterStr == TEXT("class"))
			{
				FilterType = EFilterType::Class;
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "Couldnt parse recording filter, using actor filters from settings.");
			}
		}

		if(FilterType == EFilterType::Actor || FilterType == EFilterType::Class)
		{
			TCHAR Specifier[128];
			if(FParse::Token(Str, Specifier, UE_ARRAY_COUNT(Specifier), 0))
			{
				FString const SpecifierStr = FString(Specifier).TrimStart();
				if(FilterType == EFilterType::Actor)
				{
					AActor* FoundActor = FindActorByLabel(SpecifierStr, InWorld, true);
					if(FoundActor)
					{
						Settings->ActorFilter.ActorClassesToRecord.Empty();
						FSequenceRecorder::Get().ClearQueuedRecordings();
						FSequenceRecorder::Get().AddNewQueuedRecording(FoundActor);
						FSequenceRecorder::Get().StartRecording();					
					}
					return true;
				}
				else
				{
					UClass* FoundClass = UClass::TryFindTypeSlow<UClass>(SpecifierStr);
					if(FoundClass != nullptr)
					{
						Settings->ActorFilter.ActorClassesToRecord.Empty();
						Settings->ActorFilter.ActorClassesToRecord.Add(FoundClass);
						Settings->bRecordNearbySpawnedActors = false;
						Settings->NearbyActorRecordingProximity = 0.0f;

						FSequenceRecorder::Get().ClearQueuedRecordings();

						TArray<AActor*> ActorsToRecord;
						FindActorsOfClass(FoundClass, InWorld, ActorsToRecord);

						for(AActor* ActorToRecord : ActorsToRecord)
						{
							FSequenceRecorder::Get().AddNewQueuedRecording(ActorToRecord);
						}

						FSequenceRecorder::Get().StartRecording();
						return true;
					}
					else
					{
						UE_LOGF(LogAnimation, Warning, "Couldnt parse class filter, aborting recording.");
					}
				}
			}
		}
		else
		{
			FSequenceRecorder::Get().ClearQueuedRecordings();

			TArray<AActor*> ActorsToRecord;
			if(FilterType == EFilterType::None)
			{
				for(TSubclassOf<AActor>& SubClass : Settings->ActorFilter.ActorClassesToRecord)
				{
					FindActorsOfClass(*SubClass, InWorld, ActorsToRecord);
				}
			}
			else
			{
				Settings->bRecordNearbySpawnedActors = false;
				Settings->NearbyActorRecordingProximity = 0.0f;

				Settings->ActorFilter.ActorClassesToRecord.Empty();
				Settings->ActorFilter.ActorClassesToRecord.Add(AActor::StaticClass());

				FindActorsOfClass(AActor::StaticClass(), InWorld, ActorsToRecord);
			}

			for(AActor* ActorToRecord : ActorsToRecord)
			{
				FSequenceRecorder::Get().AddNewQueuedRecording(ActorToRecord);
			}

			FSequenceRecorder::Get().StartRecording();
			return true;
		}
#endif
		return false;
	}

	bool HandleStopRecordSequenceCommand(UWorld* InWorld, const TCHAR* InStr, FOutputDevice& Ar)
	{
#if WITH_EDITOR
		FSequenceRecorder::Get().StopRecording();
		FSequenceRecorder::Get().ClearQueuedRecordings();
		return true;
#else
		return false;
#endif
	}

	// ISequenceRecorder interface
	virtual bool StartRecording(UWorld* World, const FSequenceRecorderActorFilter& ActorFilter) override
	{
		return FSequenceRecorder::Get().StartRecordingForReplay(World, ActorFilter);
	}

	virtual void StopRecording() override
	{
		FSequenceRecorder::Get().StopRecording();
	}

	virtual bool IsRecording() override
	{
		return FSequenceRecorder::Get().IsRecording();
	}

	virtual FQualifiedFrameTime GetCurrentRecordingLength() override
	{
		ULevelSequence* CurrentSequence = FSequenceRecorder::Get().GetCurrentSequence().Get();
		UMovieScene*    MovieScene      = CurrentSequence ? CurrentSequence->GetMovieScene() : nullptr;
		if (MovieScene)
		{
			return FQualifiedFrameTime(FFrameTime(UE::MovieScene::DiscreteSize(MovieScene->GetPlaybackRange())), MovieScene->GetTickResolution());
		}
		return FQualifiedFrameTime();
	}

	virtual bool StartRecording(TArrayView<AActor* const> ActorsToRecord, const FString& PathToRecordTo, const FString& SequenceName) override
	{
		if(ActorsToRecord.Num() != 0)
		{
			FSequenceRecorder::Get().ClearQueuedRecordings();
			for (AActor* Actor : ActorsToRecord)
			{
				FSequenceRecorder::Get().AddNewQueuedRecording(Actor);
			}
		}
		else if (!FSequenceRecorder::Get().HasQueuedRecordings())
		{
			if(FSlateApplication::IsInitialized())
			{
				FNotificationInfo Info(LOCTEXT("SequenceRecordingErrorActor", "Couldn't find actor to record"));
				Info.bUseLargeFont = false;

				FSlateNotificationManager::Get().AddNotification(Info);
			}

			UE_LOGF(LogAnimation, Display, "Couldn't find actor to record");
		}

		return FSequenceRecorder::Get().StartRecording(PathToRecordTo, SequenceName);
	}

	virtual void NotifyActorStartRecording(AActor* Actor)
	{
		FSequenceRecorder::Get().HandleActorSpawned(Actor);
	}

	virtual void NotifyActorStopRecording(AActor* Actor)
	{
		FSequenceRecorder::Get().HandleActorDespawned(Actor);
	}

	virtual FGuid GetRecordingGuid(AActor* Actor) const
	{
		UActorRecording* Recording = FSequenceRecorder::Get().FindRecording(Actor);
		if (Recording != nullptr)
		{
			return Recording->GetSpawnableGuid();
		}

		return FGuid();
	}

	virtual FDelegateHandle RegisterAudioRecorder(const TFunction<TUniquePtr<ISequenceAudioRecorder>()>& FactoryFunction) override
	{
		ensureMsgf(!AudioFactory, TEXT("Audio recorder already registered."));

		AudioFactory = FactoryFunction;
		AudioFactoryHandle = FDelegateHandle(FDelegateHandle::GenerateNewHandle);
		return AudioFactoryHandle;
	}

	virtual void UnregisterAudioRecorder(FDelegateHandle Handle) override
	{
		if (Handle == AudioFactoryHandle)
		{
			AudioFactory = nullptr;
			AudioFactoryHandle = FDelegateHandle();
		}
	}

	virtual bool HasAudioRecorder() const override
	{
		return AudioFactoryHandle.IsValid();
	}

	virtual TUniquePtr<ISequenceAudioRecorder> CreateAudioRecorder() const
	{
		return AudioFactory ? AudioFactory() : TUniquePtr<ISequenceAudioRecorder>();
	}

	virtual UActorRecording* QueueActorToRecord(AActor* ActorToRecord) override 
	{
		if (ActorToRecord && !FSequenceRecorder::Get().FindRecording(ActorToRecord))
		{
			return FSequenceRecorder::Get().AddNewQueuedRecording(ActorToRecord);
		}

		return nullptr;
	}

	virtual USequenceRecordingBase* QueueObjectToRecord(UObject* ObjectToRecord) override
	{
		if (ObjectToRecord && !FSequenceRecorder::Get().FindRecording(ObjectToRecord))
		{
			return FSequenceRecorder::Get().AddNewQueuedRecording(ObjectToRecord);
		}

		return nullptr;
	}	

	virtual uint32 GetTakeNumberForActor(AActor* InActor) const override
	{
		// If not using a group, take numbers aren't in use, return 0
		if (!FSequenceRecorder::Get().GetCurrentRecordingGroup().IsValid())
		{
			return 0;
		}

		if (UActorRecording* Recording = FSequenceRecorder::Get().FindRecording(InActor))
		{
			return Recording->TakeNumber;
		}

		return 0;
	}

	virtual FOnRecordingStarted& OnRecordingStarted() override { return FSequenceRecorder::Get().OnRecordingStartedDelegate; }

	virtual FOnRecordingFinished& OnRecordingFinished() override { return FSequenceRecorder::Get().OnRecordingFinishedDelegate; }

	virtual FOnRecordingGroupAdded& OnRecordingGroupAdded() override { return FSequenceRecorder::Get().OnRecordingGroupAddedDelegate; }


	virtual FString GetSequenceRecordingName() const override
	{
		return FSequenceRecorder::Get().GetSequenceRecordingName();
	}

	virtual FString GetSequenceRecordingBasePath() const override
	{
		return FSequenceRecorder::Get().GetSequenceRecordingBasePath();
	}

	static void TickSequenceRecorder(float DeltaSeconds)
	{
		if (!IsRunningDedicatedServer() && !IsRunningCommandlet())
		{
			FSequenceRecorder::Get().Tick(DeltaSeconds);
		}
	}

	/** Returns the current recording group (if any), otherwise returns nullptr. */
	virtual TWeakObjectPtr<USequenceRecorderActorGroup> GetCurrentRecordingGroup() const override
	{
		return FSequenceRecorder::Get().GetCurrentRecordingGroup();
	}

	/** Adds a new recording group and picks a default name. Returns the new recording group and sets as the current recording group. */
	virtual TWeakObjectPtr<USequenceRecorderActorGroup> AddRecordingGroup() override
	{
		return FSequenceRecorder::Get().AddRecordingGroup();
	}

	/** Removes the current recording group if any. Will make GetRecordingGroup() return nullptr. */
	virtual void RemoveCurrentRecordingGroup() override
	{
		return FSequenceRecorder::Get().RemoveCurrentRecordingGroup();
	}

	/** Duplicates the current recording group if any. Returns the new recording group and sets as the current recording group. */
	virtual TWeakObjectPtr<USequenceRecorderActorGroup> DuplicateRecordingGroup() override
	{
		return FSequenceRecorder::Get().DuplicateRecordingGroup();
	}

	/** Attempts to load a recording group from the specified name. Returns a pointer to the group if successfully loaded, otherwise nullptr. */
	virtual TWeakObjectPtr<USequenceRecorderActorGroup> LoadRecordingGroup(const FName Name) override
	{
		return FSequenceRecorder::Get().LoadRecordingGroup(Name);
	}

	/** Returns a list of names for the recording groups stored in this map. */
	virtual TArray<FName> GetRecordingGroupNames() const override
	{
		return FSequenceRecorder::Get().GetRecordingGroupNames();
	}


	/** Add an extension to the SequenceRecorder */
	virtual void AddSequenceRecorderExtender(TSharedPtr<ISequenceRecorderExtender> SequenceRecorderExternder) override
	{
		FSequenceRecorder::Get().GetSequenceRecorderExtenders().Add(SequenceRecorderExternder);

		// Rebuild the UI
		TSharedPtr<SDockTab> SequenceRecorderTabPtr = SequenceRecorderTab.Pin();
		if (SequenceRecorderTabPtr.IsValid())
		{
			SequenceRecorderTabPtr->SetContent(SNew(SSequenceRecorder));
		}
	}

	/** Remove an extension from the SequenceRecorder */
	virtual void RemoveSequenceRecorderExtender(TSharedPtr<ISequenceRecorderExtender> SequenceRecorderExternder) override
	{
		FSequenceRecorder::Get().GetSequenceRecorderExtenders().Remove(SequenceRecorderExternder);
		if (!IsEngineExitRequested())
		{
			// Rebuild the UI
			TSharedPtr<SDockTab> SequenceRecorderTabPtr = SequenceRecorderTab.Pin();
			if (SequenceRecorderTabPtr.IsValid())
			{
				SequenceRecorderTabPtr->SetContent(SNew(SSequenceRecorder));
			}
		}
	}

	virtual bool RecordSingleNodeInstanceToAnimation(USkeletalMeshComponent* PreviewComponent, UAnimSequence* NewAsset, bool bShowMessage) override
	{
		return SequenceRecorderUtils::RecordSingleNodeInstanceToAnimation(PreviewComponent, NewAsset, bShowMessage);
	}

#if WITH_EDITOR
	static UAnimSequence* HandleCaptureSingleFrameAnimSequence(USkeletalMeshComponent* Component)
	{
		FAnimationRecorder Recorder;
		Recorder.bRecordTransforms = true;
		Recorder.bRecordMorphTargets = true;
		Recorder.bRecordAttributeCurves = true;
		Recorder.bRecordMaterialCurves = true;
		if (Recorder.TriggerRecordAnimation(Component))
		{
			class UAnimSequence* Sequence = Recorder.GetAnimationObject();
			if (Sequence)
			{
				Recorder.StopRecord(false);
				return Sequence;
			}
		}

		return nullptr;
	}

	static void HandlePersonaIsRecordingActive(USkeletalMeshComponent* Component, bool& bIsRecording)
	{
		bIsRecording = FAnimationRecorderManager::Get().IsRecording(Component);
	}

	static void HandlePersonaRecord(USkeletalMeshComponent* Component)
	{
		FAnimationRecorderManager::Get().RecordAnimation(Component, FString(), FString(), GetDefault<USequenceRecorderSettings>()->DefaultAnimationSettings);
	}

	static void HandlePersonaStopRecording(USkeletalMeshComponent* Component)
	{
		FAnimationRecorderManager::Get().StopRecordingAnimation(Component);
	}

	static void HandlePersonaTickRecording(USkeletalMeshComponent* Component, float DeltaSeconds)
	{
	//	FAnimationRecorderManager::Get().Tick(Component, DeltaSeconds);
	}

	static void HandlePersonaCurrentRecording(USkeletalMeshComponent* Component, UAnimSequence*& OutSequence)
	{
		OutSequence = FAnimationRecorderManager::Get().GetCurrentlyRecordingSequence(Component);
	}

	static void HandlePersonaCurrentRecordingTime(USkeletalMeshComponent* Component, float& OutTime)
	{
		OutTime = FAnimationRecorderManager::Get().GetCurrentRecordingTime(Component);
	}

	static TSharedRef<SDockTab> SpawnSequenceRecorderTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		TSharedPtr<SDockTab> MajorTab;
		SAssignNew(MajorTab, SDockTab)
			.TabRole(ETabRole::NomadTab);

		MajorTab->SetContent(SNew(SSequenceRecorder));

		FSequenceRecorderModule& SequenceRecorder = FModuleManager::GetModuleChecked<FSequenceRecorderModule>("SequenceRecorder");
		SequenceRecorder.SequenceRecorderTab = MajorTab;

		return MajorTab.ToSharedRef();
	}

	static void DrawDebug(UCanvas* InCanvas, APlayerController* InPlayerController)
	{
		FSequenceRecorder::Get().DrawDebug(InCanvas, InPlayerController);
	}
#endif
	FDelegateHandle PostEditorTickHandle;

	FDelegateHandle DrawDebugDelegateHandle;

	FDelegateHandle LevelEditorTabManagerChangedHandle;

	FDelegateHandle PersonaLayoutExtensionsHandle;

	FDelegateHandle PersonaRegisterTabsHandle;

	TFunction<TUniquePtr<ISequenceAudioRecorder>()> AudioFactory;

	FDelegateHandle AudioFactoryHandle;

	TWeakPtr<SDockTab> SequenceRecorderTab;
public:
	FTabSpawnerEntry* TabSpawnerEntry;
};

static FAutoConsoleCommand CVarEnableSequenceRecorder(
	TEXT("SequenceRecorder"),
	TEXT("Enables the Sequence Recorder tab"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			FSequenceRecorderModule& SequenceRecorder = FModuleManager::GetModuleChecked<FSequenceRecorderModule>("SequenceRecorder");
			SequenceRecorder.TabSpawnerEntry->SetMenuType(ETabSpawnerMenuType::Enabled);
		}),
	ECVF_Default);

IMPLEMENT_MODULE( FSequenceRecorderModule, SequenceRecorder )

#undef LOCTEXT_NAMESPACE
