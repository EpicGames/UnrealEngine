// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceRecorderSettings.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Camera/CameraComponent.h"
#include "Particles/ParticleSystemComponent.h"
#include "Components/LightComponent.h"
#include "SequenceRecorder.h"
#include "CineCameraComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequenceRecorderSettings)

USequenceRecorderSettings::USequenceRecorderSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateLevelSequence = true;
	bImmersiveMode = false;
	SequenceLength = FAnimationRecordingSettings::DefaultMaximumLength;
	RecordingDelay = 4.0f;
	bAllowLooping = false;
	AnimationSubDirectory = TEXT("Animations");
	AudioSubDirectory = TEXT("Audio");
	AudioGain = 0.0f;
	AudioTrackName = NSLOCTEXT("SequenceRecorder", "DefaultAudioTrackName", "Recorded Audio");
	bReplaceRecordedAudio = true;
	bRecordNearbySpawnedActors = true;
	NearbyActorRecordingProximity = 5000.0f;
	bRecordWorldSettingsActor = true;
	bReduceKeys = true;
	bAutoSaveAsset = false;
	GlobalTimeDilation = 1.0f;
	bIgnoreTimeDilation = false;

	ClassesAndPropertiesToRecord.Add(FPropertiesToRecordForClass(USkeletalMeshComponent::StaticClass()));
	ClassesAndPropertiesToRecord.Add(FPropertiesToRecordForClass(UStaticMeshComponent::StaticClass()));
	ClassesAndPropertiesToRecord.Add(FPropertiesToRecordForClass(UParticleSystemComponent::StaticClass()));
	ClassesAndPropertiesToRecord.Add(FPropertiesToRecordForClass(ULightComponent::StaticClass()));
	ClassesAndPropertiesToRecord.Add(FPropertiesToRecordForClass(UCameraComponent::StaticClass()));
	ClassesAndPropertiesToRecord.Add(FPropertiesToRecordForClass(UCineCameraComponent::StaticClass()));
}

void USequenceRecorderSettings::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	SaveConfig();
}
