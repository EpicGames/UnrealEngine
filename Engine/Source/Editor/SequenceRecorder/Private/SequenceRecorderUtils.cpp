// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceRecorderUtils.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Actor.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "AnimationRecorder.h"
#include "ITimeManagementModule.h"
#include "Misc/App.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "LevelSequence.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "SequenceRecorderActorGroup.h"
#include "UObject/SavePackage.h"

namespace SequenceRecorderUtils
{

AActor* GetAttachment(AActor* InActor, FName& SocketName, FName& ComponentName)
{
	AActor* AttachedActor = nullptr;
	if(InActor)
	{
		USceneComponent* RootComponent = InActor->GetRootComponent();
		if(RootComponent && RootComponent->GetAttachParent() != nullptr)
		{
			AttachedActor = RootComponent->GetAttachParent()->GetOwner();
			SocketName = RootComponent->GetAttachSocketName();
			ComponentName = RootComponent->GetAttachParent()->GetFName();
		}
	}

	return AttachedActor;
}

bool RecordSingleNodeInstanceToAnimation(USkeletalMeshComponent* PreviewComponent, UAnimSequence* NewAsset, bool bShowMessage)
{
	UAnimSingleNodeInstance* SingleNodeInstance = (PreviewComponent) ? Cast<UAnimSingleNodeInstance>(PreviewComponent->GetAnimInstance()) : nullptr;
	if (SingleNodeInstance && NewAsset)
	{
		auto RecordMesh = [](USkeletalMeshComponent* InPreviewComponent, UAnimSingleNodeInstance* InSingleNodeInstance, FAnimRecorderInstance& InAnimRecorder, float CurrentTime, float Interval)
		{
			// set time
			InSingleNodeInstance->SetPosition(CurrentTime, false);
			// tick component
			InPreviewComponent->TickComponent(0.f, ELevelTick::LEVELTICK_All, nullptr);
			if (CurrentTime == 0.f)
			{
				// first frame records the current pose, so I'll have to call BeginRecording when CurrentTime == 0.f;
				InAnimRecorder.BeginRecording();
			}
			else
			{
				InAnimRecorder.Update(Interval);
			}
		};

		FAnimRecorderInstance AnimRecorder;
		FAnimationRecordingSettings Setting;
		AnimRecorder.Init(PreviewComponent, NewAsset,nullptr, Setting);
		float Length = SingleNodeInstance->GetLength();
		const FFrameRate DefaultSampleRate = Setting.SampleFrameRate.IsValid() ? Setting.SampleFrameRate : FAnimationRecordingSettings::DefaultSampleFrameRate;
		const float Interval = DefaultSampleRate.AsInterval();
		float Time = 0.f;
		for (; Time < Length; Time += Interval)
		{
			RecordMesh(PreviewComponent, SingleNodeInstance, AnimRecorder, Time, Interval);
		}

		// get the last time
		const float Remainder = (Length - (Time - Interval));
		if (Remainder >= 0.f)
		{
			RecordMesh(PreviewComponent, SingleNodeInstance, AnimRecorder, Length, Remainder);
		}

		AnimRecorder.FinishRecording(bShowMessage);
		return true;
	}

	return false;
}

FMovieSceneTimecodeSource GetTimecodeSource()
{
	return FMovieSceneTimecodeSource(FApp::GetTimecode());
}


FString MakeNewGroupName(const FString& BaseAssetPath, const FString& BaseAssetName, const TArray<FName>& ExistingGroupNames)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	const FString Dot(TEXT("."));
	const FString GroupSeparator = TEXT("_"); //@todo settings

	FString AssetName = BaseAssetName;

	int32 GroupPos = BaseAssetName.Find(GroupSeparator, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	if (GroupPos != INDEX_NONE)
	{
		AssetName = BaseAssetName.Left(GroupPos);

		// If the existing base asset name doesn't conflict, use it
		FString AssetPath = BaseAssetPath / BaseAssetName / BaseAssetName + Dot + BaseAssetName;

		if (!ExistingGroupNames.Contains(FName(*BaseAssetName)) && !AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath)).IsValid())
		{
			return BaseAssetName;
		}
	}

	const TCHAR *DriveLetters = TEXT("ABCDEFGHIJKLMNOPQRSTUVWXYZ");

	uint32 AlphaCount = 0;
	uint32 DigitSize = 1;

	do
	{
		FString AlphaString;
		for (uint32 DigitCount = 0; DigitCount < DigitSize; ++DigitCount)
		{
			AlphaString = AlphaString + DriveLetters[AlphaCount];
		}

		FString NewAssetName = AssetName + GroupSeparator + AlphaString;

		AlphaCount++;
		if (AlphaCount >= 26)
		{
			AlphaCount = 0;
			DigitSize++;
		}

		FString AssetPath = BaseAssetPath / NewAssetName / NewAssetName + Dot + NewAssetName;

		if (!ExistingGroupNames.Contains(FName(*NewAssetName)) && !AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath)).IsValid())
		{
			return NewAssetName;
		}

	} while (1) ;
}


bool ParseTakeName(const FString& InTakeName, FString& OutActorName, FString& OutSessionName, uint32& OutTakeNumber, const FString& InSessionName)
{
	const FString TakeSeparator = TEXT("_"); //@todo settings
	const int32 TakeNumDigits = 3; //@todo settings

	FString TakeName = InTakeName;

	// If the input session name is not empty, look for it in the string and extract out the session name
	if (!InSessionName.IsEmpty())
	{
		int32 SessionPos = TakeName.Find(InSessionName);
		if (SessionPos != INDEX_NONE)
		{
			TakeName.RemoveAt(SessionPos, InSessionName.Len());
			OutSessionName = InSessionName;
		}
	}

	// Split into separators
	TArray<FString> Splits;
	TakeName.ParseIntoArray(Splits, *TakeSeparator);

	// The last part is the take
	bool bHasTakeNumber = false;
	if (Splits.Num() > 0)
	{
		OutTakeNumber = FCString::Atoi(*Splits[Splits.Num()-1]);
		Splits.Pop(EAllowShrinking::Yes);
		bHasTakeNumber = true;
	}

	// The middle is the session name
	if (Splits.Num() > 0 && OutSessionName.IsEmpty())
	{
		OutSessionName = Splits[Splits.Num()-1];
		Splits.Pop(EAllowShrinking::Yes);
	}

	// The rest is the actor name
	if (Splits.Num() > 0)
	{
		OutActorName = FString::Join(Splits, *TakeSeparator);
	}

	return bHasTakeNumber;
}

void CreateCameraCutTrack(ULevelSequence* LevelSequence, const FGuid& RecordedCameraGuid, const FMovieSceneSequenceID& SequenceID)
{
	if (!RecordedCameraGuid.IsValid() || !LevelSequence)
	{
		return;
	}

	UMovieSceneTrack* CameraCutTrack = LevelSequence->GetMovieScene()->GetCameraCutTrack();
	if (CameraCutTrack && CameraCutTrack->GetAllSections().Num() > 1)
	{
		return;
	}


	if (!CameraCutTrack)
	{
		CameraCutTrack = LevelSequence->GetMovieScene()->AddCameraCutTrack(UMovieSceneCameraCutTrack::StaticClass());
	}
	else
	{
		CameraCutTrack->RemoveAllAnimationData();
	}

	UMovieSceneCameraCutSection* CameraCutSection = Cast<UMovieSceneCameraCutSection>(CameraCutTrack->CreateNewSection());
	CameraCutSection->SetCameraBindingID(UE::MovieScene::FRelativeObjectBindingID(RecordedCameraGuid, SequenceID));
	CameraCutSection->SetRange(LevelSequence->GetMovieScene()->GetPlaybackRange());
	CameraCutTrack->AddSection(*CameraCutSection);
}

void ExtendSequencePlaybackRange(ULevelSequence* LevelSequence)
{		
	UMovieScene* MovieScene = LevelSequence->GetMovieScene();
	if(MovieScene)
	{
		TRange<FFrameNumber> OriginalPlayRange = MovieScene->GetPlaybackRange();

		TRange<FFrameNumber> PlayRange = OriginalPlayRange;

		TArray<UMovieSceneSection*> MovieSceneSections = MovieScene->GetAllSections();
		for(UMovieSceneSection* Section : MovieSceneSections)
		{
			TRange<FFrameNumber> SectionRange = Section->GetRange();
			if (SectionRange.GetLowerBound().IsClosed() && SectionRange.GetUpperBound().IsClosed())
			{
				PlayRange = TRange<FFrameNumber>::Hull(PlayRange, SectionRange);
			}
		}

		MovieScene->SetPlaybackRange(TRange<FFrameNumber>(OriginalPlayRange.GetLowerBoundValue(), PlayRange.GetUpperBoundValue()));

		// Initialize the working and view range with a little bit more space
		FFrameRate  TickResolution = MovieScene->GetTickResolution();
		const double OutputViewSize = PlayRange.Size<FFrameNumber>() / TickResolution;
		const double OutputChange   = OutputViewSize * 0.1;

		TRange<double> NewRange = UE::MovieScene::ExpandRange(PlayRange / TickResolution, OutputChange);
		FMovieSceneEditorData& EditorData = MovieScene->GetEditorData();
		EditorData.ViewStart = EditorData.WorkStart = NewRange.GetLowerBoundValue();
		EditorData.ViewEnd   = EditorData.WorkEnd   = NewRange.GetUpperBoundValue();
	}
}

void SaveAsset(UObject* InObject)
{
	if (!InObject)
	{
		return;
	}

	// auto-save asset outside of the editor
	UPackage* const Package = InObject->GetOutermost();
	FString const PackageName = Package->GetName();
	FString const PackageFileName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());
	// More like a quick fix to be able to save sequence recordings in -game mode
	Package->GetMetaData();

	FSavePackageArgs SaveArgs;
	SaveArgs.TopLevelFlags = RF_Standalone;
	SaveArgs.SaveFlags = SAVE_NoError;
	UPackage::SavePackage(Package, NULL, *PackageFileName, SaveArgs);
}

void GatherTakeInfo(ULevelSequence* InLevelSequence, TArray<FTakeInfo>& TakeInfos)
{
	UMovieScene* MovieScene = InLevelSequence->GetMovieScene();

	for (auto Track : MovieScene->GetTracks())
	{
		if (Track->IsA<UMovieSceneSubTrack>())
		{
			UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(Track);
			for (auto Section : SubTrack->GetAllSections())
			{
				if (Section->IsA<UMovieSceneSubSection>())
				{
					UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
					ULevelSequence* SubSequence = Cast<ULevelSequence>(SubSection->GetSequence());
					
					FString ActorLabel;
					FString SessionName = InLevelSequence->GetName();
					uint32 TakeNumber;
					if (ParseTakeName(SubSequence->GetName(), ActorLabel, SessionName, TakeNumber, SessionName))
					{
						TakeInfos.Add(FTakeInfo(ActorLabel, TakeNumber, SubSequence));
					}
				}
			}
		}
	}
}

}
