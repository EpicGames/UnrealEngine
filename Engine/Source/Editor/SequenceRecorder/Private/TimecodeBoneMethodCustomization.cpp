// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeBoneMethodCustomization.h"

#include "AnimationRecorder.h"
#include "DetailWidgetRow.h"
#include "SSocketChooser.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailPropertyRow.h"
#include "IDetailChildrenBuilder.h"
#include "LevelEditor.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "TimecodeBoneMethodCustomization"

namespace
{

class SComponentChooserPopup : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnComponentChosen, FName );

	SLATE_BEGIN_ARGS( SComponentChooserPopup )
		: _Actor(NULL)
		{}

		/** An actor with components */
		SLATE_ARGUMENT( AActor*, Actor )

		/** Called when the text is chosen. */
		SLATE_EVENT( FOnComponentChosen, OnComponentChosen )

	SLATE_END_ARGS()

	/** Delegate to call when component is selected */
	FOnComponentChosen OnComponentChosen;

	/** List of tag names selected in the tag containers*/
	TArray< TSharedPtr<FName> > ComponentNames;

private:
	TSharedRef<ITableRow> MakeListViewWidget(TSharedPtr<FName> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew( STableRow< TSharedPtr<FName> >, OwnerTable )
				[
					SNew(STextBlock) .Text( FText::FromName(*InItem.Get()) )
				];
	}

	void OnComponentSelected(TSharedPtr<FName> InItem, ESelectInfo::Type InSelectInfo)
	{
		FSlateApplication::Get().DismissAllMenus();

		if(OnComponentChosen.IsBound())
		{
			OnComponentChosen.Execute(*InItem.Get());
		}
	}

public:
	void Construct( const FArguments& InArgs )
	{
		OnComponentChosen = InArgs._OnComponentChosen;
		AActor* Actor = InArgs._Actor;

		TInlineComponentArray<USceneComponent*> Components(Actor);

		ComponentNames.Empty();
		for(USceneComponent* Component : Components)
		{
			if (Component->HasAnySockets())
			{
				ComponentNames.Add(MakeShareable(new FName(Component->GetFName())));
			}
		}

		// Then make widget
		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Menu.Background")))
			.Padding(5)
			.Content()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 1.0f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle(TEXT("SocketChooser.TitleFont")) )
					.Text( NSLOCTEXT("ComponentChooser", "ChooseComponentLabel", "Choose Component") )
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.MaxHeight(512)
				[
					SNew(SBox)
					.WidthOverride(256)
					.Content()
					[
						SNew(SListView< TSharedPtr<FName> >)
						.ListItemsSource( &ComponentNames)
						.OnGenerateRow( this, &SComponentChooserPopup::MakeListViewWidget )
						.OnSelectionChanged( this, &SComponentChooserPopup::OnComponentSelected )
					]
				]
			]
		];
	}
};

}

TSharedRef<IPropertyTypeCustomization> FTimecodeBoneMethodCustomization::MakeInstance()
{
	return MakeShareable(new FTimecodeBoneMethodCustomization);
}

void FTimecodeBoneMethodCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.
		NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			StructPropertyHandle->CreatePropertyValueWidget()
		];
}

void FTimecodeBoneMethodCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Retrieve structure's child properties
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);
	TMap<FName, TSharedPtr< IPropertyHandle > > PropertyHandles;
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	// Retrieve special case properties
	BoneModeHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FTimecodeBoneMethod, BoneMode));
	BoneNameHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FTimecodeBoneMethod, BoneName));

	for (auto Iter(PropertyHandles.CreateConstIterator()); Iter; ++Iter)
	{
		// make the widget
		IDetailPropertyRow& PropertyRow = ChildBuilder.AddProperty(Iter.Value().ToSharedRef());

		// special customization to show interactive actor picker
		if (Iter.Value() == BoneNameHandle)
		{
			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;
			FDetailWidgetRow Row;
			PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

			auto IsEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FTimecodeBoneMethodCustomization::IsBoneNameEnabled));
			ValueWidget->SetEnabled(IsEnabled);

			PropertyRow.CustomWidget(/*bShowChildren*/ true)
			.NameContent()
			[
				NameWidget.ToSharedRef()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					ValueWidget.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.Padding(2.0f, 0.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					PropertyCustomizationHelpers::MakeInteractiveActorPicker(
						FOnGetAllowedClasses::CreateSP(this, &FTimecodeBoneMethodCustomization::OnGetAllowedClasses), 
						FOnShouldFilterActor::CreateSP(this, &FTimecodeBoneMethodCustomization::OnShouldFilterActor), 
						FOnActorSelected::CreateSP(this, &FTimecodeBoneMethodCustomization::OnActorSelected))
				]
			];
		}
	}
}

bool FTimecodeBoneMethodCustomization::IsBoneNameEnabled()
{
	uint8 BoneMode;
	BoneModeHandle->GetValue(BoneMode);

	if ((ETimecodeBoneMode)BoneMode == ETimecodeBoneMode::UserDefined)
	{
		return true;
	}

	return false;
}

void FTimecodeBoneMethodCustomization::OnGetAllowedClasses(TArray<const UClass*>& AllowedClasses)
{
	AllowedClasses.Add(AActor::StaticClass());
}

bool FTimecodeBoneMethodCustomization::OnShouldFilterActor(const AActor* const InActor)
{
	TInlineComponentArray<USceneComponent*> Components(InActor);

	for (USceneComponent* Component : Components)
	{
		if (Component->HasAnySockets())
		{
			return true;
		}
	}

	return false;
}

void FTimecodeBoneMethodCustomization::OnActorSelected(AActor* InActor)
{
	TArray<USceneComponent*> ComponentsWithSockets;
	TInlineComponentArray<USceneComponent*> Components(InActor);

	for (USceneComponent* Component : Components)
	{
		if (Component->HasAnySockets())
		{
			ComponentsWithSockets.Add(Component);
		}
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( "LevelEditor");
	TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();

	TSharedPtr<SWidget> MenuWidget;

	MenuWidget = 
		SNew(SComponentChooserPopup)
		.Actor(InActor)
		.OnComponentChosen(this, &FTimecodeBoneMethodCustomization::ActorComponentPicked, InActor);		

	// Create as context menu
	FSlateApplication::Get().PushMenu(
		LevelEditor.ToSharedRef(),
		FWidgetPath(),
		MenuWidget.ToSharedRef(),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
		);
}

void FTimecodeBoneMethodCustomization::ActorComponentPicked(FName ComponentName, AActor* InActor)
{
	USceneComponent* ComponentWithSockets = nullptr;
	if (InActor)
	{
		TInlineComponentArray<USceneComponent*> Components(InActor);
	
		for (USceneComponent* Component : Components)
		{
			if (Component->GetFName() == ComponentName)
			{
				ComponentWithSockets = Component;
				break;
			}
		}
	}

	if (ComponentWithSockets == nullptr)
	{
		return;
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( "LevelEditor");
	TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();

	TSharedPtr<SWidget> MenuWidget;

	MenuWidget = 
		SNew(SSocketChooserPopup)
		.SceneComponent(ComponentWithSockets)
		.OnSocketChosen(this, &FTimecodeBoneMethodCustomization::ActorSocketPicked, ComponentWithSockets);		

	// Create as context menu
	FSlateApplication::Get().PushMenu(
		LevelEditor.ToSharedRef(),
		FWidgetPath(),
		MenuWidget.ToSharedRef(),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
		);
}

void FTimecodeBoneMethodCustomization::ActorSocketPicked(FName SocketName, USceneComponent* InComponent)
{
	const FScopedTransaction Transaction(LOCTEXT("PickedSocketName", "Pick Socket Name"));
	ensure(BoneNameHandle->SetValue(SocketName, EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
	ensure(BoneModeHandle->SetValue((uint8)ETimecodeBoneMode::UserDefined, EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
}

#undef LOCTEXT_NAMESPACE // FTimecodeBoneStyleCustomization
