// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "IPropertyTypeCustomization.h"

class FDetailWidgetRow;
class IPropertyHandle;
class AActor;
class USceneComponent;

class FTimecodeBoneMethodCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization instance */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

protected:
 	
	TSharedPtr<IPropertyHandle> BoneModeHandle;
 	TSharedPtr<IPropertyHandle> BoneNameHandle;

private:

	bool IsBoneNameEnabled();
	void OnGetAllowedClasses(TArray<const UClass*>& AllowedClasses);
	bool OnShouldFilterActor(const AActor* const InActor);
	void OnActorSelected(AActor* InActor);
	void ActorComponentPicked(FName ComponentName, AActor* InActor);
	void ActorSocketPicked(FName SocketName, USceneComponent* InComponent);
};
