// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequenceRecordingBase.h"
#include "Misc/Guid.h"
#include "Animation/AnimationRecordingSettings.h"
#include "IMovieSceneSectionRecorder.h"
#include "ActorRecordingSettings.h"
#include "UObject/ObjectKey.h"
#include "UObject/SoftObjectPath.h"
#include "GameFramework/Actor.h"

#include "ActorRecording.generated.h"

class UAnimSequence;
class ULevelSequence;
class UMovieScene;
class USceneComponent;
class FMovieSceneAnimationSectionRecorder;

UCLASS(MinimalAPI)
class UActorRecording : public USequenceRecordingBase
{
	GENERATED_UCLASS_BODY()

public:
	/** Check whether it is worth recording this actor - i.e. is it going to affect the end result of the sequence */
	static bool IsRelevantForRecording(AActor* Actor);

	/** UItemRecordingBase override */
	virtual bool StartRecording(class ULevelSequence* CurrentSequence = nullptr, float CurrentSequenceTime = 0.0f, const FString& BaseAssetPath = FString(), const FString& SessionName = FString()) override;
	virtual bool StopRecording(class ULevelSequence* CurrentSequence = nullptr, float CurrentSequenceTime = 0.0f) override;
	virtual void Tick(ULevelSequence* CurrentSequence = nullptr, float CurrentSequenceTime = 0.0f) override;
	virtual bool IsRecording() const override;
	virtual UObject* GetObjectToRecord() const override { return GetActorToRecord(); }
	virtual bool IsActive() const override { return bActive; }

	/** Simulate a de-spawned actor */
	void InvalidateObjectToRecord();

	/** Get the Guid that identifies our spawnable in a recorded sequence */
	const FGuid& GetSpawnableGuid() const
	{
		return Guid;
	}

	/** Get the actor to record. This finds the corresponding actor in the Simulation / PIE world. */
	AActor* GetActorToRecord() const;

	/** Set the actor to record */
	void SetActorToRecord(AActor* InActor);

	/** Get the active level sequence, optionally overridden by the target level sequence */
	ULevelSequence* GetActiveLevelSequence(ULevelSequence* InLevelSequence);

	/** Get target name */
	FString GetTargetName(AActor* InActor);

	/** Get the object binding for the actor if it exists in the level sequence either as a track with the track name or a tag with the actor label */
	FGuid GetActorInSequence(AActor* InActor, ULevelSequence* CurrentSequence);

private:

	/** Get whether should duplicate the level sequence before recording into it */
	bool ShouldDuplicateLevelSequence();

	/** Check component validity for recording */
	bool ValidComponent(UActorComponent* ActorComponent) const;

	/** Adds us to a folder for better sequence organization */
	void FindOrAddFolder(UMovieScene* MovieScene);

	/** Start recording actor properties to a sequence */
	void StartRecordingActorProperties(ULevelSequence* CurrentSequence, float CurrentSequenceTime);

	/** Start recording component properties to a sequence */
	TSharedPtr<class FMovieSceneAnimationSectionRecorder> StartRecordingComponentProperties(const FName& BindingName, UActorComponent* ActorComponent, UObject* BindingContext, ULevelSequence* CurrentSequence, float CurrentSequenceTime, const FAnimationRecordingSettings& InAnimationSettings, UAnimSequence* InTargetSequence);

	/** Start recording components that are added at runtime */
	void StartRecordingNewComponents(ULevelSequence* CurrentSequence, float CurrentSequenceTime);

	/** Helper function to grab all components base actor and scene */
	void GetAllComponents(TArray<UActorComponent*>& OutArray, bool bIncludeNonCDO = true);

	/** Helper function to grab all scene components in the actor's hierarchy */
	void GetSceneComponents(TArray<UActorComponent*>& OutArray, bool bIncludeNonCDO = true);

	/** Helper to get all Non-Scene Actor Componets*/
	void GetNonSceneActorComponents(TArray<UActorComponent*>& OutArray);

	/** Sync up tracked components with the actor */
	void SyncTrackedComponents(bool bIncludeNonCDO = true);

	/** Ensure that we are recording any parents required for the specified component, and sort the specified array */
	void ProcessNewComponentArray(TInlineComponentArray<UActorComponent*>& ProspectiveComponents) const;

	/** UObject interface */
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

public:
	UPROPERTY(EditAnywhere, Category = "Actor Recording")
	FActorRecordingSettings ActorSettings;

	/** Whether this actor is active and to be recorded when the 'Record' button is pressed. */
	UPROPERTY(EditAnywhere, Category = "Actor Recording")
	bool bActive;

	/** Whether to create a level sequence for this actor recording */
	UPROPERTY(EditAnywhere, Category = "Actor Recording")
	bool bCreateLevelSequence;

	/** The level sequence to record into */
	UPROPERTY(EditAnywhere, Category = "Actor Recording", meta=(EditCondition = "bCreateLevelSequence"))
	TObjectPtr<class ULevelSequence> TargetLevelSequence;

	/** Optional target name to record to. If not specified, the actor label will be used */
	UPROPERTY(EditAnywhere, Category = "Actor Recording", meta=(EditCondition = "bCreateLevelSequence"))
	FText TargetName;

	/** Specify the take number for the new recording */
	UPROPERTY(EditAnywhere, Category = "Actor Recording", meta=(EditCondition = "bCreateLevelSequence"))
	uint32 TakeNumber;

	/** Whether we should specify the target animation or auto-create it */
	UPROPERTY(EditAnywhere, Category = "Animation Recording")
	bool bSpecifyTargetAnimation;

	/** The target animation we want to record to */
	UPROPERTY(EditAnywhere, Category = "Animation Recording", meta=(EditCondition = "bSpecifyTargetAnimation"))
	TObjectPtr<class UAnimSequence> TargetAnimation;

	/** The settings to apply to this actor's animation */
	UPROPERTY(EditAnywhere, Category = "Animation Recording")
	FAnimationRecordingSettings AnimationSettings;

	/** Whether to record to 'possessable' (i.e. level-owned) or 'spawnable' (i.e. sequence-owned) actors. Defaults to the global setting. */
	UPROPERTY(EditAnywhere, Category = "Actor Recording")
	bool bRecordToPossessable;

	/** Whether this actor recording was triggered from an actor spawn */
	bool bWasSpawnedPostRecord;

private:
	/** The actor we want to record */
	UPROPERTY(EditAnywhere, Category = "Actor Recording")
	TSoftObjectPtr<AActor> ActorToRecord;

	/** This actor's current set of section recorders */
	TArray<TSharedPtr<class IMovieSceneSectionRecorder>> SectionRecorders;

	/** Track components to check if any have changed */
	TArray<TWeakObjectPtr<UActorComponent>> TrackedComponents;

	TMap<FObjectKey, TWeakObjectPtr<UActorComponent>> DuplicatedDynamicComponents;

	/** Flag to track whether we created new components */
	bool bNewComponentAddedWhileRecording;

	/** Guid that identifies our spawnable in a recorded sequence */
	FGuid Guid;
};
