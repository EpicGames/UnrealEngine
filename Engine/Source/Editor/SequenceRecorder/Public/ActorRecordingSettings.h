// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Templates/Casts.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "ActorRecordingSettings.generated.h"

#define UE_API SEQUENCERECORDER_API

class UObject;

USTRUCT()
struct FActorRecordingSettings
{
	GENERATED_BODY()

	UE_API FActorRecordingSettings();

	UE_API FActorRecordingSettings(class UObject* InOuter);

	UE_API void CreateSettingsObjectsFromFactory();

	template <typename SettingsType>
	SettingsType* GetSettingsObject() const
	{
		CreateSettingsObjectsIfNeeded();
		for (UObject* SettingsObject : Settings)
		{
			if (SettingsType* TypedSettingsObject = Cast<SettingsType>(SettingsObject))
			{
				return TypedSettingsObject;
			}
		}

		return nullptr;
	}

private:
	UE_API void CreateSettingsObjectsIfNeeded() const;

private:
	/** External settings objects for recorders that supply them. Displayed via a details customization  */
	UPROPERTY(EditAnywhere, Category = "Actor Recording")
	TArray<TObjectPtr<UObject>> Settings;

	/** An optional outer that settings objects should be created with. */
	TWeakObjectPtr<UObject> Outer;
};

#undef UE_API
