// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "UObject/GCObject.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimationRecordingSettings.h"
#include "Components/SkinnedMeshComponent.h"
#include "Animation/AnimNotifyQueue.h"
#include "Serializers/MovieSceneAnimationSerialization.h"
#include "Misc/QualifiedFrameTime.h"
#include "Animation/AnimTypes.h"
#include "TimecodeBoneMethod.h"
#include "IAnimationRecorder.h"
#include "IAnimRecorderInstance.h"

#define UE_API SEQUENCERECORDER_API

class UAnimBoneCompressionSettings;
class UAnimNotify;
class UAnimNotifyState;
class UAnimSequence;
class USkeletalMeshComponent;

DECLARE_LOG_CATEGORY_EXTERN(AnimationSerialization, Verbose, All);

//////////////////////////////////////////////////////////////////////////
// FAnimationRecorder

// records the mesh pose to animation input
struct FAnimationRecorder 
	: public UE::AnimationRecording::IAnimationRecorder
	, public FGCObject
{
private:
	/** Frame count used to signal an unbounded animation */
	static const int32 UnBoundedFrameCount = -1;

private:
	FFrameRate RecordingRate;
	FFrameNumber MaxFrame;
	FFrameNumber LastFrame;
	double TimePassed;
	TObjectPtr<UAnimSequence> AnimationObject;
	TArray<FTransform> PreviousSpacesBases;
	FBlendedHeapCurve PreviousAnimCurves;
	FTransform PreviousComponentToWorld;
	FTransform InvInitialRootTransform;
	FTransform InitialRootTransform;
	int32 SkeletonRootIndex;

	/** Unique notifies added to this sequence during recording */
	TMap<UAnimNotify*, UAnimNotify*> UniqueNotifies;

	/** Unique notify states added to this sequence during recording */
	TMap<UAnimNotifyState*, UAnimNotifyState*> UniqueNotifyStates;

	/** This function returns the current timecode that the recorder should use to label the current frame */
	TFunction<TOptional<FQualifiedFrameTime>()> GetCurrentFrameTimeFunction;

	struct FRecordedAnimNotify
	{
		FRecordedAnimNotify(const FAnimNotifyEvent& InNewNotifyEvent, const FAnimNotifyEvent* InOriginalNotifyEvent, float InAnimNotifyStartTime, float InAnimNotifyEndTime)
			: NewNotifyEvent(InNewNotifyEvent)
			, OriginalNotifyEvent(InOriginalNotifyEvent)
			, AnimNotifyStartTime(InAnimNotifyStartTime)
			, AnimNotifyEndTime(InAnimNotifyEndTime)
			, bWasActive(true)
		{}

		/** Notify which will be added to this sequence */
		FAnimNotifyEvent NewNotifyEvent;

		/** Notify which was called on the sequence being recorded */
		const FAnimNotifyEvent* OriginalNotifyEvent;

		/** The time in the recorded animation at which the recorded notify started and ended */
		float AnimNotifyStartTime;
		float AnimNotifyEndTime;

		/** Whether this notify was active this frame */
		bool bWasActive;
	};

	/** Notify events recorded at any point, processed and inserted into animation when recording has finished */
	TArray<FRecordedAnimNotify> RecordedAnimNotifies;

	/** Currently recording notify events that have duration */
	TArray<FRecordedAnimNotify> RecordingAnimNotifies;

	static UE_API float DefaultSampleRate;

	/** Array of times recorded */
	TArray<FQualifiedFrameTime> RecordedTimes;

public:
	UE_API FAnimationRecorder();
	UE_API virtual ~FAnimationRecorder() override;

	// FGCObject interface start
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FAnimationRecorder");
	}
	// FGCObject interface end

	/** Starts recording an animation. Prompts for asset path and name via dialog if none provided */
	UE_API bool TriggerRecordAnimation(USkeletalMeshComponent* Component);
	UE_API bool TriggerRecordAnimation(USkeletalMeshComponent* Component, const FString& AssetPath, const FString& AssetName);

	// IAnimationRecorder interface start
	UE_API virtual void StartRecord(USkeletalMeshComponent* Component, UAnimSequence* InAnimationObject) override;
	UE_API virtual UAnimSequence* StopRecord(bool bShowMessage) override;
	UE_API virtual void UpdateRecord(USkeletalMeshComponent* Component, float DeltaTime) override;
	virtual bool InRecording() const override { return AnimationObject != nullptr; }

	/** Sets the function that should be used to get the current timecode for the current frame. */
	UE_API virtual void SetCurrentFrameTimeGetter(TFunction<TOptional<FQualifiedFrameTime>()> InGetCurrentTimeFunction) override;

	virtual const FTransform& GetInitialRootTransform() const override { return InitialRootTransform; }

	/** Process any time data captured and apply it to the bones on the given SkeletalMeshComponent. */
	UE_API virtual void ProcessRecordedTimes(UAnimSequence* AnimSequence, USkeletalMeshComponent* SkeletalMeshComponent, const FTimecodeBoneMethod& TimecodeBoneMethod, const FProcessRecordedTimeParams& TimecodeInfo) override;
	// IAnimationRecorder interface end

	UAnimSequence* GetAnimationObject() const { return AnimationObject; }
	double GetTimeRecorded() const { return TimePassed; }

	/** Sets a new sample rate & max length for this recorder. Don't call while recording. */
	UE_API void SetSampleRateAndLength(FFrameRate SampleFrameRate, float LengthInSeconds);

	UE_API bool SetAnimCompressionScheme(UAnimBoneCompressionSettings* Settings);

	UE_DEPRECATED(5.5, "Use the ProcessRecordedTimes method that takes a FProcessRecordedTimeParams struct.")
	UE_API void ProcessRecordedTimes(UAnimSequence* AnimSequence, USkeletalMeshComponent* SkeletalMeshComponent, const FString& HoursName, const FString& MinutesName, const FString& SecondsName, const FString& FramesName, const FString& SubFramesName, const FString& SlateName, const FString& Slate, const FTimecodeBoneMethod& TimecodeBoneMethod);

	/** If true, it will record root to include LocalToWorld */
	uint8 bRecordLocalToWorld :1;
	/** If true, asset will be saved to disk after recording. If false, asset will remain in mem and can be manually saved. */
	uint8 bAutoSaveAsset : 1;
	/** If true, the root bone transform will be removed from all bone transforms */
	uint8 bRemoveRootTransform : 1;
	/** If true we check delta time at beginning of recording */
	uint8 bCheckDeltaTimeAtBeginning : 1;
	/** Interpolation type for the recorded sequence */
	EAnimInterpolationType Interpolation;
	/** The interpolation mode for the recorded keys */
	ERichCurveInterpMode InterpMode;
	/** The tangent mode for the recorded keys*/
	ERichCurveTangentMode TangentMode;
	/** Serializer, if set we also store data out incrementally while running*/
	FAnimationSerializer* AnimationSerializer;
	/** Whether or not to set Retarget Source Asset*/
	uint8 bSetRetargetSourceAsset : 1;
	/** Whether or not to record transforms*/
	uint8 bRecordTransforms : 1;
	/** Whether or not to record morph targets*/
	uint8 bRecordMorphTargets : 1;
	/** Whether or not to record attribute curves*/
	uint8 bRecordAttributeCurves : 1;
	/** Whether or not to record material curves*/
	uint8 bRecordMaterialCurves : 1;
	/** If enabled, we will skip any curves that have only zero values */
	uint8 bSkipCurvesWithZeroValue : 1;
	/** If enabled, curves in ExcludeAnimationNames that already exist in the target sequence will be removed at the start of recording */
	uint8 bRemoveExcludedCurves : 1;
	/** Include list */
	TArray<FString> IncludeAnimationNames;
	/** Exclude list */
	TArray<FString> ExcludeAnimationNames;
	/** Whether or not to transact any IAnimationDataController changes */
	bool bTransactRecording;
public:
	/** Helper function to get space bases depending on leader pose component */
	static UE_API void GetBoneTransforms(USkeletalMeshComponent* Component, TArray<FTransform>& BoneTransforms);

private:
	UE_API bool Record(USkeletalMeshComponent* Component, FTransform const& ComponentToWorld, const TArray<FTransform>& SpacesBases, const FBlendedHeapCurve& AnimationCurves, int32 FrameToAdd);

	UE_API void RecordNotifies(USkeletalMeshComponent* Component, const TArray<FAnimNotifyEventReference>& AnimNotifies, float DeltaTime, float RecordTime);

	UE_API void ProcessNotifies();

	UE_API bool ShouldSkipName(const FName& InName) const;

	TArray<FBlendedHeapCurve> RecordedCurves;
	TArray<FRawAnimSequenceTrack> RawTracks;
};

//////////////////////////////////////////////////////////////////////////
// FAnimRecorderInstance

struct FAnimRecorderInstance : public UE::AnimationRecording::IAnimRecorderInstance
{
public:
	UE_API FAnimRecorderInstance();
	UE_API virtual ~FAnimRecorderInstance() override;

	UE_API void Init(USkeletalMeshComponent* InComponent, const FString& InAssetPath, const FString& InAssetName, const FAnimationRecordingSettings& InSettings);

	
	// IAnimRecorderInstance interface start
	UE_API virtual void Init(USkeletalMeshComponent* InComponent, UAnimSequence* InSequence, FAnimationSerializer *InAnimationSerializer, const FAnimationRecordingSettings& InSettings) override;
	UE_API virtual bool BeginRecording() override;
	UE_API virtual void Update(float DeltaTime) override;
	UE_API virtual void FinishRecording(bool bShowMessage = true) override;

	/** Process any time data captured and apply it to the bones on the given SkeletalMeshComponent. */
	UE_API virtual void ProcessRecordedTimes(UAnimSequence* AnimSequence, USkeletalMeshComponent* SkeletalMeshComponent, const FTimecodeBoneMethod& TimecodeBoneMethod, const FProcessRecordedTimeParams& TimecodeInfo) override;
	virtual TSharedPtr<UE::AnimationRecording::IAnimationRecorder> GetRecorder() const override { return Recorder; }
	// IAnimRecorderInstance interface end

	UE_DEPRECATED(5.5, "Use the ProcessRecordedTimes method that takes a FProcessRecordedTimeParams struct.")
	UE_API void ProcessRecordedTimes(UAnimSequence* AnimSequence, USkeletalMeshComponent* SkeletalMeshComponent, const FString& HoursName, const FString& MinutesName, const FString& SecondsName, const FString& FramesName, const FString& SubFramesName, const FString& SlateName, const FString& Slate, const FTimecodeBoneMethod& TimecodeBoneMethod);

private:
	UE_API void InitInternal(USkeletalMeshComponent* InComponent, const FAnimationRecordingSettings& Settings, FAnimationSerializer *InAnimationSerializer = nullptr);

public:
	TWeakObjectPtr<USkeletalMeshComponent> SkelComp;
	TWeakObjectPtr<UAnimSequence> Sequence;
	FString AssetPath;
	FString AssetName;

	/** Original ForcedLodModel setting on the SkelComp, so we can modify it and restore it when we are done. */
	int CachedSkelCompForcedLodModel;

	TSharedPtr<FAnimationRecorder> Recorder;

	/** Used to store/restore update flag when recording */
	EVisibilityBasedAnimTickOption CachedVisibilityBasedAnimTickOption;

	/** Used to store/restore URO when recording */
	bool bCachedEnableUpdateRateOptimizations;
};


//////////////////////////////////////////////////////////////////////////
// FAnimationRecorderManager

struct FAnimationRecorderManager
{
public:
	/** Singleton accessor */
	static UE_API FAnimationRecorderManager& Get();

	/** Destructor */
	UE_API virtual ~FAnimationRecorderManager();

	/** Starts recording an animation. */
	UE_API bool RecordAnimation(USkeletalMeshComponent* Component, const FString& AssetPath = FString(), const FString& AssetName = FString(), const FAnimationRecordingSettings& Settings = FAnimationRecordingSettings());

	UE_API bool RecordAnimation(USkeletalMeshComponent* Component, UAnimSequence* Sequence, const FAnimationRecordingSettings& Settings = FAnimationRecordingSettings());
	
	UE_API bool RecordAnimation(USkeletalMeshComponent* Component, UAnimSequence* Sequence, FAnimationSerializer *InAnimationSerializer,  const FAnimationRecordingSettings& Settings = FAnimationRecordingSettings());

	UE_API bool IsRecording(USkeletalMeshComponent* Component);

	UE_API bool IsRecording();

	UE_API UAnimSequence* GetCurrentlyRecordingSequence(USkeletalMeshComponent* Component);
	UE_API float GetCurrentRecordingTime(USkeletalMeshComponent* Component);
	UE_API void StopRecordingAnimation(USkeletalMeshComponent* Component, bool bShowMessage = true);
	UE_API void StopRecordingAllAnimations();
	UE_API const FTransform& GetInitialRootTransform(USkeletalMeshComponent* Component) const;

	UE_API void Tick(float DeltaTime);

	UE_API void Tick(USkeletalMeshComponent* Component, float DeltaTime);

	UE_API void StopRecordingDeadAnimations(bool bShowMessage = true);

private:
	/** Constructor, private - use Get() function */
	UE_API FAnimationRecorderManager();

	TArray<FAnimRecorderInstance> RecorderInstances;

	UE_API void HandleEndPIE(bool bSimulating);
};

#undef UE_API
