// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** A recorder class used to create and populate individual sections in level sequences */
class IMovieSceneSectionRecorder
{
public:
	/**
	 * Start recording a section. Should create the section and setup anything needed for recording here.
	 * @param	InObjectToRecord	The object (Actor of USceneComponent) to record.
	 * @param	InMovieScene		The movie scene we are recording to
	 * @param	InGuid				The Guid of the object in the movie scene
	 * @param	InTime				The current recording time in the movie scene
	 */
	virtual void CreateSection(UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, float InTime) = 0;

	/**
	 * Called when recording finishes and the section will no longer be recorded into.
	 * @param	InCurrentTime	The current recording time in the movie scene
	 */
	virtual void FinalizeSection(float InCurrentTime) = 0;

	/**
	 * Called each frame to record into the section.
	 * @param	InCurrentTime	The current recording time in the movie scene
	 */
	virtual void Record(float InCurrentTime) = 0;

	/**
	 * Added to deal with actor pooling implementations. Usually a recorder would track a weak ptr to the object to record
	 * and no longer record if the object became invalid. This function allows pooling implementations that do not necessarily
	 * destroy objects to deal with stopping recording correctly. 
	 * Usually it is enough to invalidate the object that this recorder is recording, hence the function name.
	 */
	virtual void InvalidateObjectToRecord() = 0;

	/**
	 * Retrieve the source object that this section recorder is recording changes to
	 * @return the source object, or nullptr if it's no longer valid
	 */
	virtual UObject* GetSourceObject() const = 0;
};
