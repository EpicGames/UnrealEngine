// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMovieSceneSectionRecorder.h"
#include "Features/IModularFeature.h"

/**  
 * Factory class interface that allows the recorder to determine what recorders to apply to 
 * actors/components/objects it is presented with.
 */
class IMovieSceneSectionRecorderFactory : public IModularFeature
{
public:
	/**
	 * Create a section recorder from this factory. 
	 * @param	InActorRecording	The actor recording that will be using this recorder
	 * @return a new property recorder, or nullptr if no recorder needs to be created given the settings
	 */
	virtual TSharedPtr<class IMovieSceneSectionRecorder> CreateSectionRecorder(const struct FActorRecordingSettings& InActorRecordingSettings) const = 0;

	/**
	 * Check whether an object can be recorded by this section recorder. If so then the actor recorder
	 * will call CreateSectionRecorder() to acquire a new instance to use in recording.
	 * 
	 * @param	InObjectToRecord	The object to check
	 * @return true if the object can be recorded by this recorder
	 */
	virtual bool CanRecordObject(class UObject* InObjectToRecord) const = 0;

	/**
	 * Create a per-recording settings object. You can access this object at record time using FActorRecordingSettings::Settings
	 * @return an object used for user settings for this recorder
	 */
	virtual UObject* CreateSettingsObject(class UObject* InOuter) const { return nullptr; }
};
