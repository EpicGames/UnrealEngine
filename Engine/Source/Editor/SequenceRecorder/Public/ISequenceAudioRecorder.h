// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

class USoundWave;

/** Sequencer version of the audio recording settings. */
struct FSequenceAudioRecorderSettings
{
	/** Directory to create the asset within (empty for transient package) */
	FDirectoryPath Directory;

	/** Name of the asset. */
	FString AssetName;

	/** Optional audio recording duration in seconds. */
	float RecordingDuration;

	/** Gain in decibels of the output asset. */
	float GainDb;

	/** Whether or not to split input channels into separate assets. */
	bool bSplitChannels;

	FSequenceAudioRecorderSettings()
		: RecordingDuration(-1.0f)
		, GainDb(0.0f)
		, bSplitChannels(false)
	{}
};

class ISequenceAudioRecorder
{
public:

	/** Virtual destructor */
	virtual ~ISequenceAudioRecorder() = default;
	
	/** Start recording audio data */
	virtual void Start(const FSequenceAudioRecorderSettings& Settings) = 0;

	/** Stop recording audio data */
	virtual void Stop(TArray<USoundWave*>& OutSoundWaves) = 0;
};
