// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Modules/ModuleInterface.h"
#include "Containers/ArrayView.h"
#include "Misc/QualifiedFrameTime.h"
#include "SequenceRecorderActorGroup.h"

class AActor;
class ISequenceAudioRecorder;
class ISequenceRecorderExtender;
class USequenceRecordingBase;
class UAnimSequence;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnRecordingStarted, class UMovieSceneSequence* /*Sequence*/);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnRecordingFinished, class UMovieSceneSequence* /*Sequence*/);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnRecordingGroupAdded, TWeakObjectPtr<USequenceRecorderActorGroup> /*ActorGroup*/);

class ISequenceRecorder : public IModuleInterface
{
public:
	/** 
	 * Start recording the passed-in actors.
	 * @param	World			The world we use to record actors
	 * @param	ActorFilter		Actor filter to gather actors spawned in this world for recording
	 * @return true if recording was successfully started
	 */
	virtual bool StartRecording(UWorld* World, const struct FSequenceRecorderActorFilter& ActorFilter) = 0;

	/** Stop recording current sequence, if any */
	virtual void StopRecording() = 0;

	/** Are we currently recording a sequence */
	virtual bool IsRecording() = 0;

	/** How long is the currently recording sequence */
	virtual FQualifiedFrameTime GetCurrentRecordingLength() = 0;

	/**
	 * Start a recording, possibly with some delay (specified by the sequence recording settings).
	 * @param	ActorsToRecord		Actors to record.
	 * @param	PathToRecordTo		Optional path to a sequence to record to. If none is specified we use the defaults in the settings.
	 * @param	SequenceName		Optional name of a sequence to record to. If none is specified we use the defaults in the settings.
	 * @return true if recording was successfully started
	*/
	virtual bool StartRecording(TArrayView<AActor* const> ActorsToRecord, const FString& PathToRecordTo = FString(), const FString& SequenceName = FString()) = 0;

	/**
	 * Start a recording, possibly with some delay (specified by the sequence recording settings).
	 * @param	ActorToRecord		Actor to record.
	 * @param	PathToRecordTo		Optional path to a sequence to record to. If none is specified we use the defaults in the settings.
	 * @param	SequenceName		Optional name of a sequence to record to. If none is specified we use the defaults in the settings.
	 * @return true if recording was successfully started
	*/
	bool StartRecording(AActor* ActorToRecord, const FString& PathToRecordTo = FString(), const FString& SequenceName = FString())
	{
		return StartRecording(MakeArrayView(&ActorToRecord, 1), PathToRecordTo, SequenceName);
	}

	/**
	 * Notify we should start recording an actor - usually used for 'actor pooling' implementations
	 * to simulate spawning. Has no effect when recording is not in progress.
	 * @param	Actor	The actor that was 'spawned'
	 */
	virtual void NotifyActorStartRecording(AActor* Actor) = 0;

	/**
	 * Notify we should stop recording an actor - usually used for 'actor pooling' implementations
	 * to simulate de-spawning. Has no effect when recording is not in progress.
	 * @param	Actor	The actor that was 'de-spawned'
	 */
	virtual void NotifyActorStopRecording(AActor* Actor) = 0;

	/**
	 * Get the spawnable Guid in the currently recording movie scene for the specified actor.
	 * @param	Actor	The Actor to check
	 * @return the spawnable Guid
	 */
	virtual FGuid GetRecordingGuid(AActor* Actor) const = 0;

	/**
	 * Register a function that will return a new audio capturer for the specified parameters
	 * @param	FactoryFunction		Function used to generate a new audio recorder
	 * @return A handle to be passed to UnregisterAudioRecorder to unregister the recorder
	 */
	virtual FDelegateHandle RegisterAudioRecorder(const TFunction<TUniquePtr<ISequenceAudioRecorder>()>& FactoryFunction) = 0;

	/**
	 * Unregister a previously registered audio recorder factory function
	 * @param	RegisteredHandle	The handle returned from RegisterAudioRecorder
	 */
	virtual void UnregisterAudioRecorder(FDelegateHandle RegisteredHandle) = 0;

	/**
	 * Check whether we have an audio recorder registered or not
	 * @return true if we have an audio recorder registered, false otherwise
	 */
	virtual bool HasAudioRecorder() const = 0;

	/**
	 * Add an actor to be recorded when the next recording pass begins
	 * @param	ActorToRecord	The actor to queue for recording	
	 */
	virtual UActorRecording* QueueActorToRecord(AActor* ActorToRecord) = 0;

	/**
	 * Add an object to be recorded when the next recording pass begins
	 * @param	ObjectToRecord	The object to queue for recording	
	 */
	virtual USequenceRecordingBase* QueueObjectToRecord(UObject* ObjectToRecord) = 0;

	/**
	 * Get the take number of an actor that is queued to record in the current group
	 * @param	InActor		The actor to fetch the take number for
	 * @return the take number for the given actor, 0 if actor isn't queued or no group is active
	 */
	virtual uint32 GetTakeNumberForActor(AActor* InActor) const = 0;

	/**
	 * Attempt to create an audio recorder
	 * @param	Settings	Settings for the audio recorder
	 * @return A valid ptr to an audio recorder or null
	 */
	virtual TUniquePtr<ISequenceAudioRecorder> CreateAudioRecorder() const = 0;

	/** Get the sequence recorder started delegate */
	virtual FOnRecordingStarted& OnRecordingStarted() = 0;

	/** Get the sequence recorder finished delegate */
	virtual FOnRecordingFinished& OnRecordingFinished() = 0;
	
	/** Get the name the of the sequence recording. */
	virtual FString GetSequenceRecordingName() const = 0;
	
	/** Get the directory that the sequence should record into. */
	virtual FString GetSequenceRecordingBasePath() const = 0;

	/** Returns the current recording group (if any), otherwise returns nullptr. */
	virtual TWeakObjectPtr<USequenceRecorderActorGroup> GetCurrentRecordingGroup() const = 0;

	/** Adds a new recording group and picks a default name. Returns the new recording group and sets as the current recording group. */
	virtual TWeakObjectPtr<USequenceRecorderActorGroup> AddRecordingGroup() = 0;

	/** Removes the current recording group if any. Will make GetRecordingGroup() return nullptr. */
	virtual void RemoveCurrentRecordingGroup() = 0;

	/** Duplicates the current recording group if any. Returns the new recording group and sets as the current recording group. */
	virtual TWeakObjectPtr<USequenceRecorderActorGroup> DuplicateRecordingGroup() = 0;

	/** Attempts to load a recording group from the specified name. Returns a pointer to the group if successfully loaded, otherwise nullptr. */
	virtual TWeakObjectPtr<USequenceRecorderActorGroup> LoadRecordingGroup(const FName Name) = 0;

	/** Returns a list of names for the recording groups stored in this map. */
	virtual TArray<FName> GetRecordingGroupNames() const = 0;

	/** Get the Recording Group added delegate */
	virtual FOnRecordingGroupAdded& OnRecordingGroupAdded() = 0;

	/** Add an extension to the SequenceRecorder */
	virtual void AddSequenceRecorderExtender(TSharedPtr<ISequenceRecorderExtender> SequenceRecorderExternder) = 0;

	/** Remove an extension from the SequenceRecorder */
	virtual void RemoveSequenceRecorderExtender(TSharedPtr<ISequenceRecorderExtender> SequenceRecorderExternder) = 0;

	/**
	* Play the current single node instance on the PreviewComponent from time [0, GetLength()], and record to NewAsset
	*
	* @param: PreviewComponent - this component should contains SingleNodeInstance with time-line based asset, currently support AnimSequence or AnimComposite
	* @param: NewAsset - this is the asset that should be recorded. This will reset all animation data internally
	* @param: bShowMessage - show a notification of new asset creation
	*/
	virtual bool RecordSingleNodeInstanceToAnimation(USkeletalMeshComponent* PreviewComponent, UAnimSequence* NewAsset, bool bShowMessage = true) = 0;
};
