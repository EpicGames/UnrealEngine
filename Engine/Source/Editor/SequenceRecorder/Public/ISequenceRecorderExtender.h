// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Views/SListView.h"

class IDetailsView;
class SCompoundWidget;
class UObject;
class USequenceRecordingBase;

class ISequenceRecorderExtender : public TSharedFromThis<ISequenceRecorderExtender>
{
public:
	virtual ~ISequenceRecorderExtender() = default;

	virtual TSharedPtr<IDetailsView> MakeSettingDetailsView() = 0;

	DECLARE_DELEGATE_OneParam(FListViewSelectionChanged, USequenceRecordingBase*);
	virtual TSharedPtr<SWidget> MakeListWidget(TSharedPtr<SListView<USequenceRecordingBase*>>& OutCreatedListView, FListViewSelectionChanged OnListViewSelectionChanged) = 0;

	virtual void SetListViewSelection(USequenceRecordingBase* InSelectedBase) = 0;

	virtual USequenceRecordingBase* AddNewQueueRecording(UObject* SequenceRecordingObjectToRecord) = 0;

	virtual void BuildQueuedRecordings(const TArray<USequenceRecordingBase*>& InQueuedRecordings) = 0;
};

