// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Math/Color.h"
#include "Math/Vector.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneSection.h"
#include "SequenceRecorderSettings.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API SEQUENCERECORDER_API

class UObject;
struct FGuid;


/** Interface for a generic property recorder */
class IMovieScenePropertyRecorder
{
public:
	virtual ~IMovieScenePropertyRecorder() = default;
	virtual void Create(UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, float InTime) = 0;

	virtual void Record(UObject* InObjectToRecord, float InCurrentTime) = 0;

	virtual void Finalize(UObject* InObjectToRecord, float InCurrentTime) = 0;
};

/** Helper struct for recording properties */
template <typename PropertyType>
struct FPropertyKey
{
	PropertyType Value;
	FFrameNumber Time;
};

/** Recorder for a simple property of type PropertyType */
template <typename PropertyType>
class FMovieScenePropertyRecorder : public IMovieScenePropertyRecorder
{
public:
	FMovieScenePropertyRecorder(const FTrackInstancePropertyBindings& InBinding)
		: Binding(InBinding)
	{
	}
	virtual ~FMovieScenePropertyRecorder() = default;

	virtual void Create(UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, float InTime) override
	{
		if (InObjectToRecord)
		{
			PreviousValue = Binding.GetCurrentValue<PropertyType>(*InObjectToRecord);
		}

		MovieSceneSection = AddSection(InObjectToRecord, InMovieScene, InGuid, InTime);
	}

	virtual void Record(UObject* InObjectToRecord, float InCurrentTime) override
	{
		if (!MovieSceneSection.IsValid())
		{
			return;
		}

		if (InObjectToRecord != nullptr)
		{
			FFrameRate   TickResolution  = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
			FFrameNumber CurrentFrame    = (InCurrentTime * TickResolution).FloorToFrame();

			MovieSceneSection->ExpandToFrame(CurrentFrame);

			PropertyType NewValue = Binding.GetCurrentValue<PropertyType>(*InObjectToRecord);
			if (ShouldAddNewKey(NewValue))
			{
				FPropertyKey<PropertyType> Key;
				Key.Time = CurrentFrame;
				Key.Value = NewValue;

				Keys.Add(Key);

				PreviousValue = NewValue;
			}
		}
	}

	virtual void Finalize(UObject* InObjectToRecord, float InCurrentTime) override
	{
		if (!MovieSceneSection.IsValid())
		{
			return;
		}

		for (const FPropertyKey<PropertyType>& Key : Keys)
		{
			AddKeyToSection(MovieSceneSection.Get(), Key);
		}

		const USequenceRecorderSettings* Settings = GetDefault<USequenceRecorderSettings>();
		if (Settings->bReduceKeys)
		{
			ReduceKeys(MovieSceneSection.Get());
		}
	}

private:
	/** 
	 * Helper function, specialized by type, used to check if we do capture-time filtering of keys based
	 * on previous values
	 */
	bool ShouldAddNewKey(const PropertyType& InNewValue) const;

	/** Helper function, specialized by type, used to add an appropriate section to the movie scene */
	class UMovieSceneSection* AddSection(UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, float InTime);

	/** Helper function, specialized by type, used to add keys to the movie scene section at Finalize() time */
	void AddKeyToSection(UMovieSceneSection* InSection, const FPropertyKey<PropertyType>& InKey);

	/** Helper function, specialized by type, used to reduce keys */
	void ReduceKeys(UMovieSceneSection* InSection);

private:
	/** Binding for this property */
	FTrackInstancePropertyBindings Binding;

	/** The keys that are being recorded */
	TArray<FPropertyKey<PropertyType>> Keys;

	/** Section we are recording */
	TWeakObjectPtr<class UMovieSceneSection> MovieSceneSection;

	/** Previous value we use to establish whether we should key */
	PropertyType PreviousValue;
};

/** Recorder for a simple property of type enum */
class FMovieScenePropertyRecorderEnum : public IMovieScenePropertyRecorder
{
public:
	FMovieScenePropertyRecorderEnum(const FTrackInstancePropertyBindings& InBinding)
		: Binding(InBinding)
	{
	}
	virtual ~FMovieScenePropertyRecorderEnum() = default;

	virtual void Create(UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, float InTime) override
	{
		if (InObjectToRecord)
		{
			PreviousValue = Binding.GetCurrentValueForEnum(*InObjectToRecord);
		}

		MovieSceneSection = AddSection(InObjectToRecord, InMovieScene, InGuid, InTime);
	}

	virtual void Record(UObject* InObjectToRecord, float InCurrentTime) override
	{
		if (!MovieSceneSection.IsValid())
		{
			return;
		}

		if (InObjectToRecord != nullptr)
		{
			FFrameRate   TickResolution  = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
			FFrameNumber CurrentFrame    = (InCurrentTime * TickResolution).FloorToFrame();

			MovieSceneSection->ExpandToFrame(CurrentFrame);

			int64 NewValue = Binding.GetCurrentValueForEnum(*InObjectToRecord);
			if (ShouldAddNewKey(NewValue))
			{
				FPropertyKey<int64> Key;
				Key.Time = CurrentFrame;
				Key.Value = NewValue;

				Keys.Add(Key);

				PreviousValue = NewValue;
			}
		}
	}

	virtual void Finalize(UObject* InObjectToRecord, float InCurrentTime) override
	{
		if (!MovieSceneSection.IsValid())
		{
			return;
		}

		for (const FPropertyKey<int64>& Key : Keys)
		{
			AddKeyToSection(MovieSceneSection.Get(), Key);
		}

		ReduceKeys(MovieSceneSection.Get());
	}

private:
	/** 
	 * Helper function, specialized by type, used to check if we do capture-time filtering of keys based
	 * on previous values
	 */
	UE_API bool ShouldAddNewKey(const int64& InNewValue) const;

	/** Helper function, specialized by type, used to add an appropriate section to the movie scene */
	UE_API class UMovieSceneSection* AddSection(UObject* InObjectToRecord, class UMovieScene* InMovieScene, const FGuid& InGuid, float InTime);

	/** Helper function, specialized by type, used to add keys to the movie scene section at Finalize() time */
	UE_API void AddKeyToSection(UMovieSceneSection* InSection, const FPropertyKey<int64>& InKey);

	/** Helper function, specialized by type, used to reduce keys */
	UE_API void ReduceKeys(UMovieSceneSection* InSection);

private:
	/** Binding for this property */
	FTrackInstancePropertyBindings Binding;

	/** The keys that are being recorded */
	TArray<FPropertyKey<int64>> Keys;

	/** Section we are recording */
	TWeakObjectPtr<class UMovieSceneSection> MovieSceneSection;

	/** Previous value we use to establish whether we should key */
	int64 PreviousValue;
};

#undef UE_API
