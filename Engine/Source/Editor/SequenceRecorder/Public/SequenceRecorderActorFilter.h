// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "SequenceRecorderActorFilter.generated.h"

USTRUCT()
struct FSequenceRecorderActorFilter
{
	GENERATED_BODY()

	/** Actor classes to accept for recording */
	UPROPERTY(EditAnywhere, Category="Actor Filter")
	TArray<TSubclassOf<AActor>> ActorClassesToRecord;
};
