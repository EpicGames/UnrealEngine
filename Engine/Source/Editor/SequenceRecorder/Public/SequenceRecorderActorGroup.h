// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "SequenceRecorderActorGroup.generated.h"

#define UE_API SEQUENCERECORDER_API

// Forward Declares
class UActorRecording;

UCLASS(MinimalAPI)
class USequenceRecorderActorGroup : public UObject
{
	GENERATED_BODY()
public:
	USequenceRecorderActorGroup()
	{
		SequenceName = TEXT("RecordedSequence");
		SequenceRecordingBasePath.Path = TEXT("/Game/Cinematics/Sequences");
		bSpecifyTargetLevelSequence = true;
		bDuplicateTargetLevelSequence = false;
		bRecordTargetLevelSequenceLength = false;
	}

	UPROPERTY(EditAnywhere, Category = "Recording Groups")
	FName GroupName;

	/** The base name of the sequence to record to. This name will also be used to auto-generate any assets created by this recording. */
	UPROPERTY(EditAnywhere, Category = "Recording Groups")
	FString SequenceName;

	/** Base path for this recording. Sub-assets will be created in subdirectories as specified */
	UPROPERTY(EditAnywhere, Category = "Recording Groups", meta = (ContentDir))
	FDirectoryPath SequenceRecordingBasePath;

	/** Whether we should specify the target level sequence or auto-create it */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Recording Groups")
	bool bSpecifyTargetLevelSequence;

	/** The level sequence to record into */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Recording Groups", meta=(EditCondition = "bSpecifyTargetLevelSequence"))
	TObjectPtr<class ULevelSequence> TargetLevelSequence;

	/** Whether we should duplicate the target level sequence and record into the duplicate */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Recording Groups", meta=(EditCondition = "bSpecifyTargetLevelSequence"))
	bool bDuplicateTargetLevelSequence;

	/** Whether we should record to the length of the target level sequence */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Recording Groups", meta=(EditCondition = "bSpecifyTargetLevelSequence"))
	bool bRecordTargetLevelSequenceLength;

	/** A list of actor recordings in this group which contains both the actors to record as well as settings for each one. */
	UPROPERTY(VisibleAnywhere, Category = "Recording Groups")
	TArray<TObjectPtr<UActorRecording>> RecordedActors;

	UE_API void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
};

/**
 * 
 */
UCLASS(MinimalAPI, hidedropdown, NotBlueprintable)
class ASequenceRecorderGroup : public AActor
{
	GENERATED_BODY()
public:
	ASequenceRecorderGroup()
		: AActor()
	{
		bListedInSceneOutliner = false;
	}

	/** AActor Interface */
	virtual bool IsEditorOnly() const final
	{
		return true;
	}
	/** End AActor Interface */

	TWeakObjectPtr<USequenceRecorderActorGroup> FindActorGroup(const FName& Name)
	{
		for (USequenceRecorderActorGroup* Group : ActorGroups)
		{
			if (Group && Group->GroupName == Name)
			{
				return Group;
			}
		}

		return nullptr;
	}

public:

	UPROPERTY()
	TArray<TObjectPtr<USequenceRecorderActorGroup>> ActorGroups;
};

#undef UE_API
