// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "SequenceRecorderBlueprintLibrary.generated.h"

#define UE_API SEQUENCERECORDER_API

class AActor;

UCLASS(MinimalAPI, meta=(ScriptName="SequenceRecorderLibrary"))
class USequenceRecorderBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	/** 
	 * Start recording the passed-in actors to a level sequence.
	 * @param	ActorsToRecord	The actors to record
	 */
	UFUNCTION(BlueprintCallable, Category="Sequence Recording")
	static UE_API void StartRecordingSequence(const TArray<AActor*>& ActorsToRecord);

	/** Are we currently recording a sequence */
	UFUNCTION(BlueprintPure, Category="Sequence Recording")
	static UE_API bool IsRecordingSequence();

	/** Stop recording the current sequence, if any */
	UFUNCTION(BlueprintCallable, Category="Sequence Recording")
	static UE_API void StopRecordingSequence();
};

#undef UE_API
