// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Animation/AnimationRecordingSettings.h"
#include "SequenceRecorderActorFilter.h"
#include "SequenceRecorderSettings.generated.h"

#define UE_API SEQUENCERECORDER_API

class ALevelSequenceActor;
class ULevelSequence;

/** Enum denoting if (and how) to record audio */
UENUM()
enum class EAudioRecordingMode : uint8
{
	None 			UMETA(DisplayName="Don't Record Audio"),
	AudioTrack		UMETA(DisplayName="Into Audio Track"),
};

USTRUCT()
struct FPropertiesToRecordForActorClass
{
	GENERATED_BODY()

	FPropertiesToRecordForActorClass()
	{}

	FPropertiesToRecordForActorClass(TSubclassOf<AActor> InClass)
		: Class(InClass)
	{}

	/** The class of the actor we can record */
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Sequence Recording")
	TSubclassOf<AActor> Class;

	/** List of properties we want to record for this class */
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Sequence Recording")
	TArray<FName> Properties;
};

USTRUCT()
struct FPropertiesToRecordForClass
{
	GENERATED_BODY()

	FPropertiesToRecordForClass()
	{}

	FPropertiesToRecordForClass(TSubclassOf<UActorComponent> InClass)
		: Class(InClass)
	{}

	/** The class of the object we can record */
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Sequence Recording")
	TSubclassOf<UActorComponent> Class;

	/** List of properties we want to record for this class */
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Sequence Recording")
	TArray<FName> Properties;
};

USTRUCT()
struct FSettingsForActorClass
{
	GENERATED_BODY()

	/** The class of the actor we want to record */
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Sequence Recording")
	TSubclassOf<AActor> Class;

	/** Whether to record to 'possessable' (i.e. level-owned) or 'spawnable' (i.e. sequence-owned) actors. */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	bool bRecordToPossessable = false;
};

UCLASS(MinimalAPI, config=Editor)
class USequenceRecorderSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;

public:
	/** Whether to create a level sequence when recording. Actors and animations will be inserted into this sequence */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	bool bCreateLevelSequence;

	/** Whether to maximize the viewport when recording */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	bool bImmersiveMode;

	/** The length of the recorded sequence */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording", meta = (ClampMin="0.0", UIMin = "0.0"))
	float SequenceLength;

	/** Delay that we will use before starting recording */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording", meta = (ClampMin="0.0", UIMin = "0.0", ClampMax="9.0", UIMax = "9.0"))
	float RecordingDelay;

	/** Allow the recording to be looped. Subsequence recorded assets will be saved to unique filenames. */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	bool bAllowLooping;

	/** Global Time dilation to set the world to when recording starts. Useful for playing back a scene in slow motion. */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "10.0"))
	float GlobalTimeDilation;

	/** Should Sequence Recorder ignore global time dilation? If true recorded animations will only be as long as they would have been without global time dilation. */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	bool bIgnoreTimeDilation;

	/** The name of the subdirectory animations will be placed in. Leave this empty to place into the same directory as the sequence base path */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	FString AnimationSubDirectory;

	/** Whether to record audio alongside animation or not */
	UPROPERTY(Config, EditAnywhere, Category = "Audio Recording")
	EAudioRecordingMode RecordAudio;

	/** Gain in decibels to apply to recorded audio */
	UPROPERTY(Config, EditAnywhere, Category = "Audio Recording", meta = (ClampMin="0.0", UIMin = "0.0"))
	float AudioGain;

	/** Whether or not to split mic channels into separate audio tracks. If not true, a max of 2 input channels is supported. */
	UPROPERTY(Config, EditAnywhere, Category = "Audio Recording")
	bool bSplitAudioChannelsIntoSeparateTracks;

	/** Replace existing recorded audio with any newly recorded audio */
	UPROPERTY(Config, EditAnywhere, Category = "Audio Recording")
	bool bReplaceRecordedAudio;

	/** Name of the recorded audio track name */
	UPROPERTY(Config, EditAnywhere, Category = "Audio Recording")
	FText AudioTrackName;

	/** The name of the subdirectory audio will be placed in. Leave this empty to place into the same directory as the sequence base path */
	UPROPERTY(Config, EditAnywhere, Category = "Audio Recording")
	FString AudioSubDirectory;

	/** Whether to record nearby spawned actors. If an actor matches a class in the ActorFilter, this state will be bypassed. */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	bool bRecordNearbySpawnedActors;

	/** Proximity to currently recorded actors to record newly spawned actors. */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording", meta = (ClampMin="0.0", UIMin = "0.0"))
	float NearbyActorRecordingProximity;

	/** Whether to record the world settings actor in the sequence (some games use this to attach world SFX) */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	bool bRecordWorldSettingsActor;

	/** Whether to remove keyframes within a tolerance from the recorded tracks */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	bool bReduceKeys;

	/** Whether to auto-save asset when recording is completed. Defaults to false */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	bool bAutoSaveAsset;

	/** Filter to check spawned actors against to see if they should be recorded */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	FSequenceRecorderActorFilter ActorFilter;

	/** Sequence actors to trigger playback on when recording starts (e.g. for recording synchronized sequences) */
	UPROPERTY(Transient, EditAnywhere, Category = "Sequence Recording")
	TArray<TWeakObjectPtr<class ALevelSequenceActor>> LevelSequenceActorsToTrigger;

	/** Default animation settings which are used to initialize all new actor recording's animation settings */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	FAnimationRecordingSettings DefaultAnimationSettings;

	/** Whether to record actors that are spawned by sequencer itself (this is usually disabled as results can be unexpected) */
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Sequence Recording")
	bool bRecordSequencerSpawnedActors;

	/** The properties to record for specified classes. Component classes specified here will be recorded. If an actor does not contain one of these classes it will be ignored. */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	TArray<FPropertiesToRecordForClass> ClassesAndPropertiesToRecord;

	/** The properties to record for specified actors. Actor classes specified here will be recorded. If an actor does not contain one of these properties it will be ignored. */
	UPROPERTY(Config, EditAnywhere, Category = "Sequence Recording")
	TArray<FPropertiesToRecordForActorClass> ActorsAndPropertiesToRecord;

	/** Settings applied to actors of a specified class */
	UPROPERTY(Config, EditAnywhere, AdvancedDisplay, Category = "Sequence Recording")
	TArray<FSettingsForActorClass> PerActorSettings;
};

#undef UE_API
