// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineLogs.h"
#include "MovieScene.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ObjectTools.h"

class AActor;
class Error;
class UAnimSequence;
class ULevelSequence;

namespace SequenceRecorderUtils
{

/**
 * Utility function that creates an asset with the specified asset path and name.
 * If the asset cannot be created (as one already exists), we try to postfix the asset
 * name until we can successfully create the asset.
 */
template<typename AssetType>
static AssetType* MakeNewAsset(const FString& BaseAssetPath, const FString& BaseAssetName)
{
	const FString Dot(TEXT("."));
	FString AssetPath = BaseAssetPath;
	FString AssetName = BaseAssetName;

	AssetPath /= AssetName;
	AssetPath += Dot + AssetName;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));

	// if object with same name exists, try a different name until we don't find one
	int32 ExtensionIndex = 0;
	while(AssetData.IsValid() && AssetData.GetClass() == AssetType::StaticClass())
	{
		AssetName = FString::Printf(TEXT("%s_%d"), *BaseAssetName, ExtensionIndex);
		AssetPath = (BaseAssetPath / AssetName) + Dot + AssetName;
		AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));

		ExtensionIndex++;
	}

	// Create the new asset in the package we just made
	AssetPath = (BaseAssetPath / AssetName);

	FString FileName;
	if(FPackageName::TryConvertLongPackageNameToFilename(AssetPath, FileName))
	{
		UObject* Package = CreatePackage( *AssetPath);
		return NewObject<AssetType>(Package, *AssetName, RF_Public | RF_Standalone);	
	}

	UE_LOGF(LogAnimation, Error, "Couldnt create file for package %ls", *AssetPath);

	return nullptr;
}

static UObject* DuplicateAsset(const FString& BaseAssetPath, const FString& BaseAssetName, UObject* ObjectToDuplicate)
{
	// Verify the source object
	if ( !ObjectToDuplicate )
	{
		return nullptr;
	}

	const FString Dot(TEXT("."));
	FString AssetPath = BaseAssetPath;
	FString AssetName = BaseAssetName;

	AssetPath /= AssetName;
	AssetPath += Dot + AssetName;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));

	// if object with same name exists, try a different name until we don't find one
	int32 ExtensionIndex = 0;
	while(AssetData.IsValid() && AssetData.GetClass() == ObjectToDuplicate->GetClass())
	{
		AssetName = FString::Printf(TEXT("%s_%d"), *BaseAssetName, ExtensionIndex);
		AssetPath = (BaseAssetPath / AssetName) + Dot + AssetName;
		AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));

		ExtensionIndex++;
	}

	const FString PackageName = BaseAssetPath + TEXT("/") + AssetName;

	ObjectTools::FPackageGroupName PGN;
	PGN.PackageName = PackageName;
	PGN.GroupName = TEXT("");
	PGN.ObjectName = AssetName;

	TSet<UPackage*> ObjectsUserRefusedToFullyLoad;
	UObject* NewObject = ObjectTools::DuplicateSingleObject(ObjectToDuplicate, PGN, ObjectsUserRefusedToFullyLoad);

	return NewObject;
}

static FString MakeNewAssetName(const FString& BaseAssetPath, const FString& BaseAssetName)
{
	const FString Dot(TEXT("."));
	FString AssetPath = BaseAssetPath;
	FString AssetName = BaseAssetName;

	AssetPath /= AssetName;
	AssetPath += Dot + AssetName;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));

	// if object with same name exists, try a different name until we don't find one
	int32 ExtensionIndex = 0;
	while (AssetData.IsValid())
	{
		AssetName = FString::Printf(TEXT("%s_%d"), *BaseAssetName, ExtensionIndex);
		AssetPath = (BaseAssetPath / AssetName) + Dot + AssetName;
		AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));

		ExtensionIndex++;
	}

	return AssetName;
}

static FString MakeTakeName(const FString& ActorName, const FString& SessionName, uint32 TakeNumber)
{
	const FString TakeSeparator = TEXT("_"); //@todo settings
	const int32 TakeNumDigits = 3; //@todo settings

	FString TakeName = ActorName + TakeSeparator + SessionName + TakeSeparator + FString::Printf(TEXT("%0*d"), TakeNumDigits, TakeNumber);

	return TakeName;
}

static bool DoesTakeExist(const FString& AssetPath, const FString& ActorName, const FString& SessionName, uint32 TakeNumber)
{
	FString TakeName = MakeTakeName(ActorName, SessionName, TakeNumber);
	FString TakePath = AssetPath / TakeName;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	const FString Dot(TEXT("."));
	FString FullPath = TakePath + Dot + TakeName;
	FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(FullPath));

	return AssetData.IsValid();
}

static int32 GetNewTakeNumber(const FString& AssetPath, const FString& ActorName, const FString& SessionName, uint32 TakeNumber)
{
	while (DoesTakeExist(AssetPath, ActorName, SessionName, TakeNumber))
	{
		TakeNumber++;
	}

	return TakeNumber;
}

/** Make an actor recording group name that is unique to the parent and to level sequence assets */
SEQUENCERECORDER_API FString MakeNewGroupName(const FString& BaseAssetPath, const FString& BaseAssetName, const TArray<FName>& ExistingGroupNames);

/** Parse the take number into its multiple parts */
SEQUENCERECORDER_API bool ParseTakeName(const FString& InTakeName, FString& OutActorName, FString& OutSessionName, uint32& OutTakeNumber, const FString& InSessionName = FString());

/** Create a camera cut track for the recorded camera */
SEQUENCERECORDER_API void CreateCameraCutTrack(ULevelSequence* LevelSequence, const FGuid& RecordedCameraGuid, const FMovieSceneSequenceID& SequenceID);

/** Extend the level sequence playback range to encompass the section ranges */
SEQUENCERECORDER_API void ExtendSequencePlaybackRange(ULevelSequence* LevelSequence);

/** Save the asset */
SEQUENCERECORDER_API void SaveAsset(UObject* InObject);

struct FTakeInfo
{
	FTakeInfo(const FString& InActorLabel, uint32 InTakeNumber, ULevelSequence* InTargetLevelSequence)
		: ActorLabel(InActorLabel)
		, TakeNumber(InTakeNumber)
		, TargetLevelSequence(InTargetLevelSequence)
	{}

	FString ActorLabel;
	uint32 TakeNumber;
	ULevelSequence* TargetLevelSequence; 
};

SEQUENCERECORDER_API void GatherTakeInfo(ULevelSequence* LevelSequence, TArray<FTakeInfo>& TakeInfos);

/** Helper function - check whether our component hierarchy has some attachment outside of its owned components */
SEQUENCERECORDER_API AActor* GetAttachment(AActor* InActor, FName& SocketName, FName& ComponentName);

/** 
 * Play the current single node instance on the PreviewComponent from time [0, GetLength()], and record to NewAsset
 * 
 * @param: PreviewComponent - this component should contains SingleNodeInstance with time-line based asset, currently support AnimSequence or AnimComposite
 * @param: NewAsset - this is the asset that should be recorded. This will reset all animation data internally
 * @param: bShowMessage - show a notification of new asset creation
 */
SEQUENCERECORDER_API bool RecordSingleNodeInstanceToAnimation(USkeletalMeshComponent* PreviewComponent, UAnimSequence* NewAsset, bool bShowMessage = true);

/**
 * Get timecode source
 */
SEQUENCERECORDER_API FMovieSceneTimecodeSource GetTimecodeSource();

}
