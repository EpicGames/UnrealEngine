// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "SequenceRecordingBase.generated.h"

class ULevelSequence;
class UMovieScene;

UCLASS(MinimalAPI, abstract)
class USequenceRecordingBase : public UObject
{
	GENERATED_BODY()

public:
	/** Start this queued recording. Sequence can be nullptr */
	virtual bool StartRecording(class ULevelSequence* CurrentSequence = nullptr, float CurrentSequenceTime = 0.0f, const FString& BaseAssetPath = FString(), const FString& SessionName = FString()) PURE_VIRTUAL(USequenceRecordingBase::StartRecording, return false;);

	/** Stop this recording. Has no effect if we are not currently recording. Sequence can be nullptr */
	virtual bool StopRecording(class ULevelSequence* CurrentSequence = nullptr, float CurrentSequenceTime = 0.0f) PURE_VIRTUAL(USequenceRecordingBase::StopRecording, return false;);

	/** Tick this recording */
	virtual void Tick(ULevelSequence* CurrentSequence = nullptr, float CurrentSequenceTime = 0.0f) PURE_VIRTUAL(USequenceRecordingBase::Tick, );

	/** Whether we are currently recording */
	virtual bool IsRecording() const PURE_VIRTUAL(USequenceRecordingBase::IsRecording, return false;);

	/** Get the item to record. */
	virtual UObject* GetObjectToRecord() const PURE_VIRTUAL(USequenceRecordingBase::GetObjectToRecord, return nullptr;);

	/** Get the item to record. */
	virtual bool IsActive() const PURE_VIRTUAL(USequenceRecordingBase::IsActive, return false;);
};

