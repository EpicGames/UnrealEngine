// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class SequenceRecorder : ModuleRules
	{
		public SequenceRecorder(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"TimeManagement",
                    "SerializedRecorderInterface",
					"SequenceRuntimeRecorder"

                }
            );

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"SlateCore",
					"Slate",
					"InputCore",
					"Engine",
					"EditorFramework",
					"UnrealEd",
					"EditorStyle",
					"Projects",
					"LevelEditor",
					"WorkspaceMenuStructure",
					"PropertyEditor",
					"MovieScene",
					"MovieSceneTracks",
					"LevelSequence",
					"NetworkReplayStreaming",
					"AssetRegistry",
					"CinematicCamera",
                    "EditorWidgets",
                    "Kismet",
                    "LiveLinkInterface",
					"SceneOutliner",
					"Sequencer"
                }
                );

            PrivateIncludePathModuleNames.AddRange(
                new string[]
                {
                    "Persona",
                }
                );

            DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
                    "Persona",
                }
				);

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				// Add __WINDOWS_WASAPI__ so that RtAudio compiles with WASAPI
				PublicDefinitions.Add("__WINDOWS_WASAPI__");
			}
		}
	}
}
